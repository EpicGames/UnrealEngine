// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InstallBundleManager : ModuleRules
{
	public InstallBundleManager(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"IoStoreOnDemandCore",
				"Json"
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"Json"
			}
		);
	}
}
