// Copyright Epic Games, Inc. All Rights Reserved.

#include "BundlePrereqCombinedStatusHelper.h"
#include "Containers/Ticker.h"
#include "InstallBundleManagerPrivate.h"
#include "InstallBundleUtils.h"
#include "Stats/Stats.h"
#include "Algo/Transform.h"

const TCHAR* LexToString(FInstallBundleCombinedProgressTracker::ECombinedBundleStatus Status)
{
	static const TCHAR* Strings[] =
	{
		TEXT("Unknown"),
		TEXT("Initializing"),
		TEXT("Updating"),
		TEXT("Finishing"),
		TEXT("Finished"),
		TEXT("Count")
	};

	static_assert(InstallBundleUtil::CastToUnderlying(FInstallBundleCombinedProgressTracker::ECombinedBundleStatus::Count) == UE_ARRAY_COUNT(Strings) - 1, "");
	return Strings[InstallBundleUtil::CastToUnderlying(Status)];
}

FInstallBundleCombinedProgressTracker::FInstallBundleCombinedProgressTracker(bool bAutoTick /*= true*/, TUniqueFunction<void(const FCombinedProgress&)> InOnTick /*= nullptr*/)
	: OnTick(MoveTemp(InOnTick))
{
	SetupDelegates(bAutoTick);
}

FInstallBundleCombinedProgressTracker::~FInstallBundleCombinedProgressTracker()
{
	CleanUpDelegates();
}

FInstallBundleCombinedProgressTracker::FInstallBundleCombinedProgressTracker(const FInstallBundleCombinedProgressTracker& Other)
{
	*this = Other;
}

FInstallBundleCombinedProgressTracker::FInstallBundleCombinedProgressTracker(FInstallBundleCombinedProgressTracker&& Other)
{
	*this = MoveTemp(Other);
}

FInstallBundleCombinedProgressTracker& FInstallBundleCombinedProgressTracker::operator=(const FInstallBundleCombinedProgressTracker& Other)
{
	if (this != &Other)
	{
		//Just copy all this data
		RequiredBundleNames = Other.RequiredBundleNames;
		BundleStatusCache = Other.BundleStatusCache;
		CachedBundleWeights = Other.CachedBundleWeights;
		CurrentCombinedProgress = Other.CurrentCombinedProgress;
		InstallBundleManager = Other.InstallBundleManager;
		
		//Don't copy TickHandle as we want to setup our own here
		SetupDelegates(Other.TickHandle.IsValid());
	}
	
	return *this;
}

FInstallBundleCombinedProgressTracker& FInstallBundleCombinedProgressTracker::operator=(FInstallBundleCombinedProgressTracker&& Other)
{
	if (this != &Other)
	{
		//Just copy small data
		CurrentCombinedProgress = Other.CurrentCombinedProgress;
		InstallBundleManager = Other.InstallBundleManager;

		//Move bigger data
		RequiredBundleNames = MoveTemp(Other.RequiredBundleNames);
		BundleStatusCache = MoveTemp(Other.BundleStatusCache);
		CachedBundleWeights = MoveTemp(Other.CachedBundleWeights);

		//Prevent other from having callbacks now that its information is gone
		Other.CleanUpDelegates();
	
		//Don't copy TickHandle as we want to setup our own here
		SetupDelegates(Other.TickHandle.IsValid());
	}
	
	return *this;
}

void FInstallBundleCombinedProgressTracker::SetupDelegates(bool bAutoTick)
{
	CleanUpDelegates();
	
	IInstallBundleManager::InstallBundleCompleteDelegate.AddRaw(this, &FInstallBundleCombinedProgressTracker::OnBundleInstallComplete);
	IInstallBundleManager::PausedBundleDelegate.AddRaw(this, &FInstallBundleCombinedProgressTracker::OnBundleInstallPauseChanged);
	if (bAutoTick)
	{
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FInstallBundleCombinedProgressTracker::Tick));
	}
}

void FInstallBundleCombinedProgressTracker::CleanUpDelegates()
{
	IInstallBundleManager::InstallBundleCompleteDelegate.RemoveAll(this);
	IInstallBundleManager::PausedBundleDelegate.RemoveAll(this);
	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle.Reset();
	}
}

void FInstallBundleCombinedProgressTracker::SetBundlesToTrackFromContentState(const FInstallBundleCombinedContentState& BundleContentState, TArrayView<FName> BundlesToTrack)
{
	RequiredBundleNames.Empty();
	CachedBundleWeights.Empty();
	BundleStatusCache.Empty();

	//Go through all bundles until we hit a non-zero weight bundle. 
	//This is to help catch instances where we pass in all zero weight bundles to track and need
	//to thus calculate their weight dynamically based on everything having even weight
	bool bAreAllBundlesZeroWeight = true;
	for (const TPair<FName, FInstallBundleContentState>& IndividualBundlePair : BundleContentState.IndividualBundleStates)
	{
		const FInstallBundleContentState& BundleState = IndividualBundlePair.Value;
		if (BundleState.Weight > SMALL_NUMBER)
		{
			bAreAllBundlesZeroWeight = false;
			break;
		}
	}

	bool bBundleNeedsUpdate = false;
	float TotalWeight = 0.0f;
	for (const FName& Bundle : BundlesToTrack)
	{
		const FInstallBundleContentState* BundleState = BundleContentState.IndividualBundleStates.Find(Bundle);
		if (ensureAlwaysMsgf(BundleState, TEXT("Trying to track unknown bundle %s"), *Bundle.ToString()))
		{
			//Filter out any bundles with effectively 0 weight (unless all bundles are 0 weight)
			if (!bAreAllBundlesZeroWeight && (BundleState->Weight <= SMALL_NUMBER))
			{
				continue;
			}

			//Track if we need any kind of bundle updates
			if (BundleState->State == EInstallBundleInstallState::NotInstalled || BundleState->State == EInstallBundleInstallState::NeedsUpdate)
			{
				bBundleNeedsUpdate = true;
			}

			//Save required bundles and their weights
			RequiredBundleNames.Add(Bundle);

			//If all bundles are zero weight, just treat this weight as 1 so everything ends up with 1 weight and is evenly distributed
			float Weight = bAreAllBundlesZeroWeight ? 1.0f : BundleState->Weight;
			CachedBundleWeights.FindOrAdd(Bundle) = Weight;

			TotalWeight += Weight;
		}
	}

	CurrentCombinedProgress.bBundleRequiresUpdate = bBundleNeedsUpdate;

	if (TotalWeight > 0.0f)
	{
		for (TPair<FName, float>& BundleWeightPair : CachedBundleWeights)
		{
			BundleWeightPair.Value /= TotalWeight;
		}
	}

	// If no bundles to track, we are done
	if (RequiredBundleNames.Num() == 0)
	{
		CurrentCombinedProgress.BackgroundDownloadProgressPercent = 1.0f;
		CurrentCombinedProgress.InstallOnlyProgressPercent = 1.0f;
		CurrentCombinedProgress.ProgressPercent = 1.0f;
		CurrentCombinedProgress.CombinedStatus = ECombinedBundleStatus::Finished;
	}

	//Go ahead and calculate initial values from the Bundle Cache
	UpdateBundleCache();
}

void FInstallBundleCombinedProgressTracker::UpdateBundleCache()
{
	//if we haven't already set this up, lets try to set it now
	if (nullptr == InstallBundleManager)
	{
		InstallBundleManager = IInstallBundleManager::GetPlatformInstallBundleManager();
	}
	
	TSharedPtr<IInstallBundleManager> PinnedBundleManger = InstallBundleManager.Pin();
	if (ensureAlwaysMsgf((nullptr != PinnedBundleManger), TEXT("Invalid InstallBundleManager during UpdateBundleCache! Needs to be valid during run!")))
	{
		for (FName& BundleName : RequiredBundleNames)
		{
			TOptional<FInstallBundleProgress> BundleProgress = PinnedBundleManger->GetBundleProgress(BundleName);
			
			//Copy progress to the cache as long as we have progress to copy.
			if (BundleProgress.IsSet())
			{
				BundleStatusCache.Add(BundleName, BundleProgress.GetValue());
			}
		}
	}
}

void FInstallBundleCombinedProgressTracker::UpdateCombinedStatus()
{
	if (RequiredBundleNames.Num() == 0)
		return;

	UpdateCombinedProgress();
	
	EInstallBundleStatus EarliestBundleState = EInstallBundleStatus::Count;
	EInstallBundlePauseFlags CombinedPauseFlags = EInstallBundlePauseFlags::None;
	bool bIsAnythingPaused = false;
	bool bIsAnythingFinishing = false;
	
	//if we don't yet have a bundle status cache entry for a particular requirement
	//then we can't yet tell what work is required on that bundle yet. We need to go ahead and make sure we don't
	//show a status like "Installed" before we know what state that bundle is in. Make sure we show at LEAST
	//updating in that case, so start with Downloading since that is the first Updating case.
	//However if all bundle progress is finished, don't just sit showing 100% and Updating when we could potentially
	//be showing Finishing progress
	if ((BundleStatusCache.Num() < RequiredBundleNames.Num())
		&& (BundleStatusCache.Num() > 0)
		&& (CurrentCombinedProgress.ProgressPercent < 1.0f))
	{
		EarliestBundleState = EInstallBundleStatus::Updating;
	}
	
	float EarliestFinishingPercent = 1.0f;
	for (const TPair<FName,FInstallBundleProgress>& BundlePair : BundleStatusCache)
	{
		if (BundlePair.Value.Status < EarliestBundleState)
		{
			EarliestBundleState = BundlePair.Value.Status;
		}

		if (!bIsAnythingFinishing && BundlePair.Value.Status == EInstallBundleStatus::Finishing)
		{
			EarliestFinishingPercent = BundlePair.Value.Finishing_Percent;
			bIsAnythingFinishing = true;
		}
		
		bIsAnythingPaused = bIsAnythingPaused || BundlePair.Value.PauseFlags != EInstallBundlePauseFlags::None;
		CombinedPauseFlags |= BundlePair.Value.PauseFlags;
	}
	
	//if we have any paused bundles, and we have any bundle that isn't finished installed, we are Paused
	//if everything is installed ignore the pause flags as we completed after pausing the bundles
	CurrentCombinedProgress.bIsPaused = (bIsAnythingPaused && (EarliestBundleState < EInstallBundleStatus::Ready));
	if(CurrentCombinedProgress.bIsPaused)
	{
		CurrentCombinedProgress.CombinedPauseFlags = CombinedPauseFlags;
	}
	else
	{
		CurrentCombinedProgress.CombinedPauseFlags = EInstallBundlePauseFlags::None;
	}
	
	//if the bundle does not need an update, all the phases we go through don't support pausing (Mounting ,Compiling Shaders, etc)
	//Otherwise start with True and override those specific cases bellow
	CurrentCombinedProgress.bDoesCurrentStateSupportPausing = CurrentCombinedProgress.bBundleRequiresUpdate;
	
	if ((EarliestBundleState == EInstallBundleStatus::Requested) || (EarliestBundleState == EInstallBundleStatus::Count))
	{
		CurrentCombinedProgress.CombinedStatus = ECombinedBundleStatus::Initializing;
	}
	else if (EarliestBundleState <= EInstallBundleStatus::Updating)
	{
		CurrentCombinedProgress.CombinedStatus = ECombinedBundleStatus::Updating;
	}
	else if (EarliestBundleState <= EInstallBundleStatus::Finishing)
	{
		//Handles the case where one of our Bundles was finishing and we have finished everything else.
		//Now just shows our earliest bundle that is finishing.
		if (bIsAnythingFinishing)
		{
			CurrentCombinedProgress.CombinedStatus = ECombinedBundleStatus::Finishing;
			CurrentCombinedProgress.ProgressPercent = EarliestFinishingPercent;
		}
		else
		{
			CurrentCombinedProgress.CombinedStatus = ECombinedBundleStatus::Updating;
		}
	}
	else if (EarliestBundleState == EInstallBundleStatus::Ready)
	{
		CurrentCombinedProgress.CombinedStatus = ECombinedBundleStatus::Finished;
		CurrentCombinedProgress.bDoesCurrentStateSupportPausing = false;
	}
	else
	{
		CurrentCombinedProgress.CombinedStatus = ECombinedBundleStatus::Unknown;
	}
}

void FInstallBundleCombinedProgressTracker::UpdateCombinedProgress()
{
	float AllBundleProgressPercent = 0.f;
	float AllBundleDownloadProgressPercent = 0.f;
	float AllBundleInstallOnlyProgressPercent = 0.f;
	int32 InstalledBundlesCount = 0;
	int32 TotalBundlesCount = 0;
	CurrentCombinedProgress.Stats.Empty();
	
	ensureAlwaysMsgf((CachedBundleWeights.Num() >= BundleStatusCache.Num()), TEXT("Missing Cache Entries for BundleWeights!Any missing bundles will have 0 for their progress!"));
	
	for (const TPair<FName,FInstallBundleProgress>& BundleStatusPair : BundleStatusCache)
	{
		const float* FoundWeight = CachedBundleWeights.Find(BundleStatusPair.Key);
		if (ensureAlwaysMsgf((nullptr != FoundWeight), TEXT("Found missing entry for BundleWeight! Bundle %s does not have a weight entry!"), *(BundleStatusPair.Key.ToString())))
		{
			AllBundleProgressPercent += ((*FoundWeight) * BundleStatusPair.Value.Install_Percent);
			AllBundleDownloadProgressPercent += ((*FoundWeight) * BundleStatusPair.Value.BackgroundDownload_Percent);
			AllBundleInstallOnlyProgressPercent += ((*FoundWeight) * BundleStatusPair.Value.InstallOnly_Percent);
			if (FMath::IsNearlyEqual(BundleStatusPair.Value.Install_Percent, 1.0f))
			{
				InstalledBundlesCount++;
			}
			TotalBundlesCount++;

			CurrentCombinedProgress.Stats.Append(BundleStatusPair.Value.Stats);
		}
	}

	CurrentCombinedProgress.ProgressPercent = FMath::Clamp(AllBundleProgressPercent, 0.f, 1.0f);
	CurrentCombinedProgress.BackgroundDownloadProgressPercent = FMath::Clamp(AllBundleDownloadProgressPercent, 0.f, 1.0f);
	CurrentCombinedProgress.InstallOnlyProgressPercent = FMath::Clamp(AllBundleInstallOnlyProgressPercent, 0.f, 1.0f);
	CurrentCombinedProgress.InstalledBundlesCount = InstalledBundlesCount;
	CurrentCombinedProgress.TotalBundlesCount = TotalBundlesCount;
}

bool FInstallBundleCombinedProgressTracker::Tick(float dt)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FBundlePrereqCombinedStatusHelper_Tick);
	UpdateBundleCache();
	UpdateCombinedStatus();
	
	if (OnTick)
	{
		OnTick(CurrentCombinedProgress);
	}

	//just always keep ticking
	return true;
}

const FInstallBundleCombinedProgressTracker::FCombinedProgress& FInstallBundleCombinedProgressTracker::GetCurrentCombinedProgress() const
{
	return CurrentCombinedProgress;
}

void FInstallBundleCombinedProgressTracker::OnBundleInstallComplete(FInstallBundleRequestResultInfo CompletedBundleInfo)
{
	const FName CompletedBundleName = CompletedBundleInfo.BundleName;
	const bool bBundleCompletedSuccessfully = (CompletedBundleInfo.Result == EInstallBundleResult::OK);
	const bool bWasRequiredBundle = RequiredBundleNames.Contains(CompletedBundleInfo.BundleName);
	
	if (bBundleCompletedSuccessfully && bWasRequiredBundle)
	{
		//Make sure our BundleCache shows this as finished all the way
		FInstallBundleProgress& BundleCacheInfo = BundleStatusCache.FindOrAdd(CompletedBundleName);
		BundleCacheInfo.Status = EInstallBundleStatus::Ready;
		
		TSharedPtr<IInstallBundleManager> PinnedBundleManger = InstallBundleManager.Pin();
		if (ensureAlwaysMsgf((nullptr != PinnedBundleManger), TEXT("Invalid InstallBundleManager during OnBundleInstallComplete! Needs to be valid during run!")))
		{
			TOptional<FInstallBundleProgress> BundleProgress = PinnedBundleManger->GetBundleProgress(CompletedBundleName);
			if (ensureAlwaysMsgf(BundleProgress.IsSet(), TEXT("Expected to find BundleProgress for completed bundle, but did not. Leaving old progress values")))
			{
				BundleCacheInfo = BundleProgress.GetValue();
			}
		}
	}
}

// It's not really necessary to have this, but it allows for a fallback if GetBundleProgress() returns null.
// Normally that shouldn't happen, but right now its handy while I refactor bundle progress.
void FInstallBundleCombinedProgressTracker::OnBundleInstallPauseChanged(FInstallBundlePauseInfo PauseInfo)
{
	const bool bWasRequiredBundle = RequiredBundleNames.Contains(PauseInfo.BundleName);
	if (bWasRequiredBundle)
	{
		FInstallBundleProgress& BundleCacheInfo = BundleStatusCache.FindOrAdd(PauseInfo.BundleName);
		BundleCacheInfo.PauseFlags = PauseInfo.PauseFlags;
	}
}
