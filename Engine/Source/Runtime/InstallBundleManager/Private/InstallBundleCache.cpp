// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleCache.h"
#include "InstallBundleManagerPrivate.h"

#define INSTALLBUNDLE_CACHE_CHECK_INVARIANTS (DO_CHECK && 0)
#define INSTALLBUNDLE_CACHE_DUMP_INFO (0)

FInstallBundleCache::~FInstallBundleCache()
{
}

void FInstallBundleCache::Init(FInstallBundleCacheInitInfo InitInfo)
{
	CacheName = InitInfo.CacheName;
	TotalSize = InitInfo.Size;
}

void FInstallBundleCache::AddOrUpdateBundle(FInstallBundleSourceType Source, const FInstallBundleCacheBundleInfo& AddInfo)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_AddOrUpdateBundle);

	FPerSourceBundleCacheInfo& Info = PerSourceCacheInfo.FindOrAdd(AddInfo.BundleName).FindOrAdd(Source);
	Info.FullInstallSize = AddInfo.FullInstallSize;
	Info.InstallOverheadSize = AddInfo.InstallOverheadSize;
	Info.CurrentInstallSize = AddInfo.CurrentInstallSize;
	Info.TimeStamp = AddInfo.TimeStamp;
	Info.AgeScalar = FMath::Clamp(AddInfo.AgeScalar, 0.1, 1.0);

	UpdateCacheInfoFromSourceInfo(AddInfo.BundleName);

	CheckInvariants();
}

void FInstallBundleCache::RemoveBundle(FInstallBundleSourceType Source, FName BundleName)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_RemoveBundle);

	TMap<FInstallBundleSourceType, FPerSourceBundleCacheInfo>* SourcesMap = PerSourceCacheInfo.Find(BundleName);
	if (SourcesMap)
	{
		SourcesMap->Remove(Source);

		UpdateCacheInfoFromSourceInfo(BundleName);

		CheckInvariants();
	}
}

TOptional<FInstallBundleCacheBundleInfo> FInstallBundleCache::GetBundleInfo(FName BundleName) const
{
	TOptional<FInstallBundleCacheBundleInfo> Ret;

	if (FBundleCacheInfo* Info = CacheInfo.Find(BundleName))
	{
		FInstallBundleCacheBundleInfo& OutInfo = Ret.Emplace();
		OutInfo.BundleName = BundleName;
		OutInfo.FullInstallSize = Info->FullInstallSize;
		OutInfo.InstallOverheadSize = Info->InstallOverheadSize;
		OutInfo.CurrentInstallSize = Info->CurrentInstallSize;
		OutInfo.TimeStamp = Info->TimeStamp;
		OutInfo.AgeScalar = Info->AgeScalar;
	}

	return Ret;
}

TOptional<FInstallBundleCacheBundleInfo> FInstallBundleCache::GetBundleInfo(FInstallBundleSourceType Source, FName BundleName) const
{
	TOptional<FInstallBundleCacheBundleInfo> Ret;

	const TMap<FInstallBundleSourceType, FPerSourceBundleCacheInfo>* SourcesMap = PerSourceCacheInfo.Find(BundleName);
	if (SourcesMap)
	{
		const FPerSourceBundleCacheInfo* SourceInfo = SourcesMap->Find(Source);
		if (SourceInfo)
		{
			FInstallBundleCacheBundleInfo& OutInfo = Ret.Emplace();
			OutInfo.BundleName = BundleName;
			OutInfo.FullInstallSize = SourceInfo->FullInstallSize;
			OutInfo.InstallOverheadSize = SourceInfo->InstallOverheadSize;
			OutInfo.CurrentInstallSize = SourceInfo->CurrentInstallSize;
			OutInfo.TimeStamp = SourceInfo->TimeStamp;
			OutInfo.AgeScalar = SourceInfo->AgeScalar;
		}
	}

	return Ret;
}

uint64 FInstallBundleCache::GetSize() const
{
	return TotalSize;
}

uint64 FInstallBundleCache::GetUsedSize() const
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_GetUsedSize);

	uint64 UsedSize = 0;
	for (const TPair<FName, FBundleCacheInfo>& Pair : CacheInfo)
	{
		UsedSize += Pair.Value.GetSize();
	}

	return UsedSize;
}

uint64 FInstallBundleCache::GetFreeSpaceInternal(uint64 UsedSize) const
{
	if (UsedSize > TotalSize)
		return 0;

	return TotalSize - UsedSize;
}

uint64 FInstallBundleCache::GetFreeSpace() const
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_GetFreeSpace);

	uint64 UsedSize = GetUsedSize();
	return GetFreeSpaceInternal(UsedSize);
}

FInstallBundleCacheReserveResult FInstallBundleCache::Reserve(FName BundleName)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_Reserve);

	FInstallBundleCacheReserveResult Result;

	FBundleCacheInfo* BundleInfo = CacheInfo.Find(BundleName);
	if (BundleInfo == nullptr)
	{
		Result.Result = EInstallBundleCacheReserveResult::Success;
		return Result;
	}

	if (BundleInfo->State == ECacheState::PendingEvict)
	{
		Result.Result = EInstallBundleCacheReserveResult::Fail_PendingEvict;
		return Result;
	}

	if (BundleInfo->State == ECacheState::Reserved)
	{
		Result.Result = EInstallBundleCacheReserveResult::Success;
		return Result;
	}

	const uint64 SizeNeeded = (BundleInfo->FullInstallSize <= BundleInfo->CurrentInstallSize) ?
		BundleInfo->InstallOverheadSize :
		BundleInfo->InstallOverheadSize + (BundleInfo->FullInstallSize - BundleInfo->CurrentInstallSize);
	const uint64 UsedSize = GetUsedSize();
	if (TotalSize >= UsedSize + SizeNeeded)
	{
		BundleInfo->State = ECacheState::Reserved;
		Result.Result = EInstallBundleCacheReserveResult::Success;

		return Result;
	}

	Result.Result = EInstallBundleCacheReserveResult::Fail_NeedsEvict;

	// TODO: Bundles that have BundleSize > 0 or are PendingEvict should be 
	// sorted to the beginning.  We should be able to stop iterating sooner in that case.
	CacheInfo.ValueSort(FCacheSortPredicate());

	uint64 CanFreeSpace = 0;
	for (const TPair<FName, FBundleCacheInfo>& Pair : CacheInfo)
	{
		if(Pair.Key == BundleName)
			continue;

		if(Pair.Value.State == ECacheState::Reserved)
			continue;

		uint64 BundleSize = Pair.Value.GetSize();
		if (BundleSize > 0)
		{
			check(UsedSize >= CanFreeSpace);
			if (TotalSize < UsedSize - CanFreeSpace + SizeNeeded)
			{
				CanFreeSpace += BundleSize;
				TArray<FInstallBundleSourceType>& SourcesToEvictFrom = Result.BundlesToEvict.Add(Pair.Key);
				PerSourceCacheInfo.FindChecked(Pair.Key).GenerateKeyArray(SourcesToEvictFrom);
			}
		}
		else if (Pair.Value.State == ECacheState::PendingEvict)
		{
			// Bundle manager must wait for all previous pending evictions to complete
			// to ensure that there is actually enough free space in the cache
			// before installing a bundle
			TArray<FInstallBundleSourceType>& SourcesToEvictFrom = Result.BundlesToEvict.Add(Pair.Key);
			PerSourceCacheInfo.FindChecked(Pair.Key).GenerateKeyArray(SourcesToEvictFrom);
		}
	}

	check(UsedSize >= CanFreeSpace);
	if (TotalSize < UsedSize - CanFreeSpace + SizeNeeded)
	{
		Result.Result = EInstallBundleCacheReserveResult::Fail_CacheFull;
	}
	else
	{
		check(Result.BundlesToEvict.Num() > 0);
	}

#if INSTALLBUNDLE_CACHE_DUMP_INFO
	GetStats(EInstallBundleCacheDumpToLog::Default, true);
#endif // INSTALLBUNDLE_CACHE_DUMP_INFO

	return Result;
}

FInstallBundleCacheFlushResult FInstallBundleCache::Flush(FInstallBundleSourceType* Source /*= nullptr*/)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_Flush);

	FInstallBundleCacheFlushResult Result;

	for (const TPair<FName, FBundleCacheInfo>& Pair : CacheInfo)
	{
		if (Pair.Value.State == ECacheState::Reserved)
			continue;

		if (Pair.Value.CurrentInstallSize == 0)
			continue;

		if (Source)
		{
			if (PerSourceCacheInfo.FindChecked(Pair.Key).Contains(*Source))
			{
				TArray<FInstallBundleSourceType>& SourcesToEvictFrom = Result.BundlesToEvict.Add(Pair.Key);
				SourcesToEvictFrom.Add(*Source);
			}
		}
		else
		{
			TArray<FInstallBundleSourceType>& SourcesToEvictFrom = Result.BundlesToEvict.Add(Pair.Key);
			PerSourceCacheInfo.FindChecked(Pair.Key).GenerateKeyArray(SourcesToEvictFrom);
		}
	}

#if INSTALLBUNDLE_CACHE_DUMP_INFO
	GetStats(EInstallBundleCacheDumpToLog::Default, true);
#endif // INSTALLBUNDLE_CACHE_DUMP_INFO

	return Result;
}

bool FInstallBundleCache::Contains(FName BundleName) const
{
	return CacheInfo.Contains(BundleName);
}

bool FInstallBundleCache::Contains(FInstallBundleSourceType Source, FName BundleName) const
{
	const TMap<FInstallBundleSourceType, FPerSourceBundleCacheInfo>* SourcesMap = PerSourceCacheInfo.Find(BundleName);
	if (SourcesMap)
	{
		return SourcesMap->Contains(Source);
	}

	return false;
}

bool FInstallBundleCache::IsReserved(FName BundleName) const
{
	const FBundleCacheInfo* BundleInfo = CacheInfo.Find(BundleName);
	if (BundleInfo)
	{
		return BundleInfo->State == ECacheState::Reserved;
	}

	return false;
}

bool FInstallBundleCache::Release(FName BundleName)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_Release);

	FBundleCacheInfo* BundleInfo = CacheInfo.Find(BundleName);
	if (BundleInfo == nullptr)
	{
		return true;
	}

	if (BundleInfo->State == ECacheState::Released)
	{
		return true;
	}

	if (BundleInfo->State == ECacheState::Reserved)
	{
		BundleInfo->State = ECacheState::Released;
		return true;
	}

	return false;
}

bool FInstallBundleCache::SetPendingEvict(FName BundleName)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_SetPendingEvict);

	FBundleCacheInfo* BundleInfo = CacheInfo.Find(BundleName);
	if (BundleInfo == nullptr)
	{
		return true;
	}

	if (BundleInfo->State == ECacheState::PendingEvict)
	{
		return true;
	}

	if (BundleInfo->State == ECacheState::Released)
	{
		BundleInfo->State = ECacheState::PendingEvict;
		return true;
	}

	return false;
}

bool FInstallBundleCache::ClearPendingEvict(FName BundleName)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_ClearPendingEvict);

	FBundleCacheInfo* BundleInfo = CacheInfo.Find(BundleName);
	if (BundleInfo == nullptr)
	{
		return true;
	}

	if (BundleInfo->State == ECacheState::Released)
	{
		return true;
	}

	if (BundleInfo->State == ECacheState::PendingEvict)
	{
		BundleInfo->State = ECacheState::Released;
		return true;
	}

	return false;
}

void FInstallBundleCache::HintRequested(FName BundleName, bool bRequested)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_HintRequested);

	FBundleCacheInfo* BundleInfo = CacheInfo.Find(BundleName);
	if (BundleInfo)
	{
		if (bRequested)
		{
			BundleInfo->HintReqeustedCount += 1;
		}
		else
		{
			BundleInfo->HintReqeustedCount -= 1;
			check(BundleInfo->HintReqeustedCount >= 0);
		}
	}
}

void FInstallBundleCache::CheckInvariants() const
{
#if INSTALLBUNDLE_CACHE_CHECK_INVARIANTS

	check(PerSourceCacheInfo.Num() == CacheInfo.Num());

	for (const TPair<FName, FBundleCacheInfo>& CachePair : CacheInfo)
	{
		const TMap<FInstallBundleSourceType, FPerSourceBundleCacheInfo>* SourcesMap = PerSourceCacheInfo.Find(CachePair.Key);
		check(SourcesMap);

		uint64 FullInstallSize = 0;
		uint64 CurrentInstallSize = 0;
		for (const TPair<FInstallBundleSourceType, FPerSourceBundleCacheInfo>& Pair : *SourcesMap)
		{
			FullInstallSize += Pair.Value.FullInstallSize;
			CurrentInstallSize += Pair.Value.CurrentInstallSize;
		}

		check(CachePair.Value.FullInstallSize == FullInstallSize);
		check(CachePair.Value.CurrentInstallSize == CurrentInstallSize);
	}

#endif // INSTALLBUNDLE_CACHE_CHECK_INVARIANTS
}

FInstallBundleCacheStats FInstallBundleCache::GetStats(EInstallBundleCacheStatsFlags Flags /*= EInstallBundleCacheStatsFlags::None*/, bool bVerbose /*= false*/) const
{
	FInstallBundleCacheStats Stats;
	Stats.CacheName = CacheName;
	Stats.MaxSize = TotalSize;

	const bool bDumpToLog = EnumHasAnyFlags(Flags, EInstallBundleCacheStatsFlags::DumpToLog);
	const bool bDumpToResults = EnumHasAnyFlags(Flags, EInstallBundleCacheStatsFlags::DumpToResults);

	if (bDumpToLog)
	{
		UE_LOGF(LogInstallBundleManager, Display, "");
		UE_LOGF(LogInstallBundleManager, Display, "* Install Bundle Cache Stats %ls", *CacheName.ToString());

		// Dump info in eviction order
		CacheInfo.ValueSort(FCacheSortPredicate());
	}

	auto AddStats = [&Stats, bDumpToResults](FName BundleName, const FBundleCacheInfo& Info)
	{
		Stats.UsedSize += Info.GetSize();

		if (Info.State == ECacheState::Reserved)
		{
			Stats.ReservedSize += Info.CurrentInstallSize;
		}

		if (bDumpToResults)
		{
			Stats.BundleStats.Emplace(FInstallBundleCacheBundleStats
			{
				.BundleName = BundleName,
				.FullInstallSize = Info.FullInstallSize,
				.InstallOverheadSize = Info.InstallOverheadSize,
				.CurrentInstallSize = Info.CurrentInstallSize,
				.TimeStamp = Info.TimeStamp,
				.AgeScalar = Info.AgeScalar,
				.bReserved = Info.State == ECacheState::Reserved,
			});
		}
	};

	// Note: Yes this verbosity is correct and very non-intuitive

#define INSTALLBUNDLECACHE_DEFAULT_LOG(Verbosity) \
	if (Info.CurrentInstallSize > 0 || Info.State != ECacheState::Released) \
	{ \
		UE_LOGF(LogInstallBundleManager, Verbosity, "* \tbundle %ls", *BundleName.ToString()) \
		UE_LOG(LogInstallBundleManager, Verbosity, TEXT("* \t\tfull size: %" UINT64_FMT), Info.FullInstallSize) \
		UE_LOG(LogInstallBundleManager, Verbosity, TEXT("* \t\toverhead size: %" UINT64_FMT), Info.InstallOverheadSize) \
		UE_LOG(LogInstallBundleManager, Verbosity, TEXT("* \t\tcurrent size: %" UINT64_FMT), Info.CurrentInstallSize) \
		UE_LOGF(LogInstallBundleManager, Verbosity, "* \t\treserved: %ls", (Info.State == ECacheState::Reserved) ? TEXT("true") : TEXT("false")) \
		UE_LOGF(LogInstallBundleManager, Verbosity, "* \t\ttimestamp: %ls", *Info.TimeStamp.ToString()) \
		UE_LOGF(LogInstallBundleManager, Verbosity, "* \t\tage scale: %f", Info.AgeScalar) \
	}

#define INSTALLBUNDLECACHE_CSV_HEADER_LOG(Verbosity) \
	UE_LOGF(LogInstallBundleManager, Verbosity, "* \tbundle, full size, overhead size, current size, diff, reserved, timestamp, age scale")

#define INSTALLBUNDLECACHE_CSV_LOG(Verbosity) \
	if (Info.CurrentInstallSize > 0 || Info.State != ECacheState::Released) \
	{ \
		const TCHAR* Diff = TEXT("="); \
		if (Info.FullInstallSize > Info.CurrentInstallSize) Diff = TEXT(">"); \
		else if(Info.FullInstallSize < Info.CurrentInstallSize) Diff = TEXT("<"); \
		UE_LOG(LogInstallBundleManager, Verbosity, TEXT("* \t%s, %" UINT64_FMT ", %" UINT64_FMT ", %" UINT64_FMT ", %s, %s, %s, %f"), *BundleName.ToString(), Info.FullInstallSize, Info.InstallOverheadSize, Info.CurrentInstallSize, Diff, (Info.State == ECacheState::Reserved) ? TEXT("true") : TEXT("false"), *Info.TimeStamp.ToString(), Info.AgeScalar) \
	}

	auto DumpToLog_Default = [](FName BundleName, const FBundleCacheInfo& Info)
	{
		INSTALLBUNDLECACHE_DEFAULT_LOG(Verbose);
	};

	auto DumpToLog_DefaultVerbose = [](FName BundleName, const FBundleCacheInfo& Info)
	{
		INSTALLBUNDLECACHE_DEFAULT_LOG(Display);
	};

	auto DumpToLog_CSV = [](FName BundleName, const FBundleCacheInfo& Info)
	{
		INSTALLBUNDLECACHE_CSV_LOG(Verbose);
	};

	auto DumpToLog_CSVVerbose = [](FName BundleName, const FBundleCacheInfo& Info)
	{
		INSTALLBUNDLECACHE_CSV_LOG(Display);
	};

	if (!bDumpToLog)
	{
		for (const TPair<FName, FBundleCacheInfo>& CachePair : CacheInfo)
		{
			const FBundleCacheInfo& Info = CachePair.Value;
			AddStats(CachePair.Key, Info);
		}
	}
	else if (EnumHasAnyFlags(Flags, EInstallBundleCacheStatsFlags::CSVFormat))
	{
		if (bVerbose)
		{
			INSTALLBUNDLECACHE_CSV_HEADER_LOG(Display);

			for (const TPair<FName, FBundleCacheInfo>& CachePair : CacheInfo)
			{
				const FBundleCacheInfo& Info = CachePair.Value;
				AddStats(CachePair.Key, Info);
				DumpToLog_CSVVerbose(CachePair.Key, Info);
			}
		}
		else
		{
			INSTALLBUNDLECACHE_CSV_HEADER_LOG(Verbose);

			for (const TPair<FName, FBundleCacheInfo>& CachePair : CacheInfo)
			{
				const FBundleCacheInfo& Info = CachePair.Value;
				AddStats(CachePair.Key, Info);
				DumpToLog_CSV(CachePair.Key, Info);
			}
		}
	}
	else
	{
		if (bVerbose)
		{
			for (const TPair<FName, FBundleCacheInfo>& CachePair : CacheInfo)
			{
				const FBundleCacheInfo& Info = CachePair.Value;
				AddStats(CachePair.Key, Info);
				DumpToLog_DefaultVerbose(CachePair.Key, Info);
			}
		}
		else
		{
			for (const TPair<FName, FBundleCacheInfo>& CachePair : CacheInfo)
			{
				const FBundleCacheInfo& Info = CachePair.Value;
				AddStats(CachePair.Key, Info);
				DumpToLog_Default(CachePair.Key, Info);
			}
		}
	}

#undef INSTALLBUNDLECACHE_CSV_LOG
#undef INSTALLBUNDLECACHE_CSV_HEADER_LOG
#undef INSTALLBUNDLECACHE_DEFAULT_LOG

	Stats.FreeSize = GetFreeSpaceInternal(Stats.UsedSize);

	if (bDumpToLog)
	{
		UE_LOG(LogInstallBundleManager, Display, TEXT("* \tsize: %" UINT64_FMT), Stats.MaxSize);
		UE_LOG(LogInstallBundleManager, Display, TEXT("* \tused: %" UINT64_FMT), Stats.UsedSize);
		UE_LOG(LogInstallBundleManager, Display, TEXT("* \treserved: %" UINT64_FMT), Stats.ReservedSize);
		UE_LOG(LogInstallBundleManager, Display, TEXT("* \tfree: %" UINT64_FMT), Stats.FreeSize);
		UE_LOGF(LogInstallBundleManager, Display, "");
	}

	return Stats;
}

void FInstallBundleCache::UpdateCacheInfoFromSourceInfo(FName BundleName)
{
	CSV_SCOPED_TIMING_STAT(InstallBundleManager, FInstallBundleCache_UpdateCacheInfoFromSourceInfo);

	TMap<FInstallBundleSourceType, FPerSourceBundleCacheInfo>* SourcesMap = PerSourceCacheInfo.Find(BundleName);
	if (SourcesMap == nullptr)
	{
		CacheInfo.Remove(BundleName);
		return;
	}

	if (SourcesMap->Num() == 0)
	{
		PerSourceCacheInfo.Remove(BundleName);
		CacheInfo.Remove(BundleName);
		return;
	}

	FDateTime TimeStamp = FDateTime::MinValue();
	double AgeScalar = 1.0;
	uint64 FullInstallSize = 0;
	uint64 InstallOverheadSize = 0;
	uint64 CurrentInstallSize = 0;
	for (const TPair<FInstallBundleSourceType, FPerSourceBundleCacheInfo>& Pair : *SourcesMap)
	{
		FullInstallSize += Pair.Value.FullInstallSize;
		InstallOverheadSize += Pair.Value.InstallOverheadSize;
		CurrentInstallSize += Pair.Value.CurrentInstallSize;

		if (Pair.Value.CurrentInstallSize > 0)
		{
			if (Pair.Value.TimeStamp > TimeStamp)
			{
				TimeStamp = Pair.Value.TimeStamp;
			}

			if (Pair.Value.AgeScalar < AgeScalar)
			{
				AgeScalar = Pair.Value.AgeScalar;
			}
		}
	}

	FBundleCacheInfo& BundleCacheInfo = CacheInfo.FindOrAdd(BundleName);
	checkf(BundleCacheInfo.FullInstallSize == FullInstallSize || BundleCacheInfo.State != ECacheState::Reserved, TEXT("Bundle %s: FullInstallSize should not be updated while a bundle is Reserved!"), *BundleName.ToString());

	BundleCacheInfo.FullInstallSize = FullInstallSize;
	BundleCacheInfo.InstallOverheadSize = InstallOverheadSize;
	BundleCacheInfo.CurrentInstallSize = CurrentInstallSize;
	BundleCacheInfo.TimeStamp = TimeStamp;
	BundleCacheInfo.AgeScalar = AgeScalar;
}
