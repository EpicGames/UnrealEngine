// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleManagerInterface.h"
#include "InstallBundleManagerModule.h"
#include "InstallBundleManagerReporting.h"

FInstallBundleManagerInitCompleteMultiDelegate IInstallBundleManager::InitCompleteDelegate;

FInstallBundleChunkDownloadMetricsMultiDelegate IInstallBundleManager::InstallBundleChunkDownloadMetricsDelegate;

FInstallBundleCompleteMultiDelegate IInstallBundleManager::InstallBundleCompleteDelegate;

FInstallBundlePausedMultiDelegate IInstallBundleManager::PausedBundleDelegate;

FInstallBundleReleasedMultiDelegate IInstallBundleManager::ReleasedDelegate;

FInstallBundleManagerOnPatchCheckComplete IInstallBundleManager::PatchCheckCompleteDelegate;

FInstallBundleManagerReportingDelegate IInstallBundleManager::ReportingDelegate;

FInstallBundleManagerAllDownloadsCompleted IInstallBundleManager::AllDownloadsCompletedDelegate;

TSharedPtr<IInstallBundleManager> IInstallBundleManager::GetPlatformInstallBundleManager()
{
	static IInstallBundleManagerModule* Module = nullptr;
	static bool bCheckedIni = false;

	if (Module)
	{
		return Module->GetInstallBundleManager();
	}

	if (!bCheckedIni && !GEngineIni.IsEmpty())
	{
		FString ModuleName;
#if WITH_EDITOR
		GConfig->GetString(TEXT("InstallBundleManager"), TEXT("EditorModuleName"), ModuleName, GEngineIni);
#else
		GConfig->GetString(TEXT("InstallBundleManager"), TEXT("ModuleName"), ModuleName, GEngineIni);
#endif // WITH_EDITOR

		if (FModuleManager::Get().ModuleExists(*ModuleName))
		{
			Module = FModuleManager::LoadModulePtr<IInstallBundleManagerModule>(*ModuleName);
		}

		bCheckedIni = true;
	}

	if (Module)
	{
		return Module->GetInstallBundleManager();
	}

	return {};
}

void IInstallBundleManager::SetContext(FName ContextName) 
{
}

bool IInstallBundleManager::CopyReportTo(FInstallManagerBundleReport& Report)
{ 
	return false; 
}

const TSharedPtr<IInstallBundleSource> IInstallBundleManager::GetBundleSource(FInstallBundleSourceType SourceType) const
{
	return {};
}

TValueOrError<FInstallBundleRequestInfo, EInstallBundleResult> IInstallBundleManager::RequestUpdateContent(FName BundleName, EInstallBundleRequestFlags Flags, ELogVerbosity::Type LogVerbosityOverride /*= ELogVerbosity::NoLogging*/,  InstallBundleUtil::FContentRequestSharedContextPtr RequestSharedContext /*= nullptr*/)
{
	return RequestUpdateContent(MakeArrayView(&BundleName, 1), Flags, LogVerbosityOverride, RequestSharedContext);
}

FDelegateHandle IInstallBundleManager::GetContentState(FName BundleName, EInstallBundleGetContentStateFlags Flags, bool bAddDependencies, FInstallBundleGetContentStateDelegate Callback, FName RequestTag /*= NAME_None*/)
{
	return GetContentState(MakeArrayView(&BundleName, 1), Flags, bAddDependencies, MoveTemp(Callback), RequestTag);
}

FDelegateHandle IInstallBundleManager::GetInstallState(FName BundleName, bool bAddDependencies, FInstallBundleGetInstallStateDelegate Callback, FName RequestTag /*= NAME_None*/)
{
	return GetInstallState(MakeArrayView(&BundleName, 1), bAddDependencies, MoveTemp(Callback), RequestTag);
}

TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> IInstallBundleManager::GetInstallStateSynchronous(FName BundleName, bool bAddDependencies) const
{
	return GetInstallStateSynchronous(MakeArrayView(&BundleName, 1), bAddDependencies);
}

TValueOrError<FInstallBundleReleaseRequestInfo, EInstallBundleResult> IInstallBundleManager::RequestReleaseContent(FName ReleaseName, EInstallBundleReleaseRequestFlags Flags, TArrayView<const FName> KeepNames /*= TArrayView<const FName>()*/, ELogVerbosity::Type LogVerbosityOverride /*= ELogVerbosity::NoLogging*/)
{
	return RequestReleaseContent(MakeArrayView(&ReleaseName, 1), Flags, KeepNames, LogVerbosityOverride);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TArray<FInstallBundleCacheStats> IInstallBundleManager::GetCacheStats(EInstallBundleCacheDumpToLog DumpToLog, ELogVerbosity::Type LogVerbosityOverride /*= ELogVerbosity::NoLogging*/)
{
	EInstallBundleCacheStatsFlags Flags = EInstallBundleCacheStatsFlags::None;
	switch (DumpToLog)
	{	
	case EInstallBundleCacheDumpToLog::CSV:
		Flags |= EInstallBundleCacheStatsFlags::CSVFormat;
		// Fallthrough
	case EInstallBundleCacheDumpToLog::Default:
		Flags |= EInstallBundleCacheStatsFlags::DumpToLog;
	}

	return GetCacheStats(Flags, LogVerbosityOverride);
}

TOptional<FInstallBundleCacheStats> IInstallBundleManager::GetCacheStats(FInstallBundleSourceOrCache SourceOrCache, EInstallBundleCacheDumpToLog DumpToLog, ELogVerbosity::Type LogVerbosityOverride /*= ELogVerbosity::NoLogging*/)
{
	EInstallBundleCacheStatsFlags Flags = EInstallBundleCacheStatsFlags::None;
	switch (DumpToLog)
	{
	case EInstallBundleCacheDumpToLog::CSV:
		Flags |= EInstallBundleCacheStatsFlags::CSVFormat;
		// Fallthrough
	case EInstallBundleCacheDumpToLog::Default:
		Flags |= EInstallBundleCacheStatsFlags::DumpToLog;
	}

	return GetCacheStats(SourceOrCache, Flags, LogVerbosityOverride);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

EInstallBundleResult IInstallBundleManager::FlushCache(FInstallBundleManagerFlushCacheCompleteDelegate Callback, ELogVerbosity::Type LogVerbosityOverride /*= ELogVerbosity::NoLogging*/)
{
	return FlushCache({}, MoveTemp(Callback), LogVerbosityOverride);
}

void IInstallBundleManager::RequestRemoveContentOnNextInit(FName RemoveName, TArrayView<const FName> KeepNames /*= TArrayView<const FName>()*/)
{
	RequestRemoveContentOnNextInit(MakeArrayView(&RemoveName, 1), KeepNames);
}

void IInstallBundleManager::CancelRequestRemoveContentOnNextInit(FName BundleName)
{
	CancelRequestRemoveContentOnNextInit(MakeArrayView(&BundleName, 1));
}

void IInstallBundleManager::CancelUpdateContent(FName BundleName)
{
	CancelUpdateContent(MakeArrayView(&BundleName, 1));
}

void IInstallBundleManager::PauseUpdateContent(FName BundleName)
{
	PauseUpdateContent(MakeArrayView(&BundleName, 1));
}

void IInstallBundleManager::ResumeUpdateContent(FName BundleName)
{
	ResumeUpdateContent(MakeArrayView(&BundleName, 1));
}

void IInstallBundleManager::UpdateContentRequestFlags(FName BundleName, EInstallBundleRequestFlags AddFlags, EInstallBundleRequestFlags RemoveFlags)
{
	UpdateContentRequestFlags(MakeArrayView(&BundleName, 1), AddFlags, RemoveFlags);
}

void IInstallBundleManager::StartPatchCheck()
{
	PatchCheckCompleteDelegate.Broadcast(EInstallBundleManagerPatchCheckResult::NoPatchRequired);
}

