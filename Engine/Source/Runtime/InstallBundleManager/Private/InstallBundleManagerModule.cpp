// Copyright Epic Games, Inc. All Rights Reserved.
#include "InstallBundleManagerModule.h"
#include "InstallBundleManagerPrivate.h"

IMPLEMENT_MODULE(FInstallBundleManagerModule, InstallBundleManager);
DEFINE_LOG_CATEGORY(LogInstallBundleManager);
CSV_DEFINE_CATEGORY_MODULE(INSTALLBUNDLEMANAGER_API, InstallBundleManager, false);
