// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/CsvProfiler.h"

DECLARE_LOG_CATEGORY_EXTERN(LogInstallBundleManager, Display, All);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(INSTALLBUNDLEMANAGER_API, InstallBundleManager);
