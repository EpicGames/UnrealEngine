// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleManagerReporting.h"
#include "InstallBundleManagerPrivate.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Algo/Find.h"

uint64 FInstallBundleReport::TotalDownloadSize() const
{
	if (!State.IsSet())
	{
		return 0;
	}
	if (const FStateUpdatable* StateUpdatable = State->TryGet<FStateUpdatable>())
	{
		return StateUpdatable->DownloadSize;
	}
	if (const FStateUpdating* StateUpdating = State->TryGet<FStateUpdating>())
	{
		return StateUpdating->DownloadSize;
	}
	if (const FStateInstalling* StateInstalling = State->TryGet<FStateInstalling>())
	{
		return StateInstalling->DownloadedSize;
	}
	if (const FStateUpdated* StateUpdated = State->TryGet<FStateUpdated>())
	{
		return StateUpdated->DownloadedSize;
	}
	checkf(false, TEXT("Unsupported state type in TotalDownloadSize()"));
	return 0;
}

uint64 FInstallBundleReport::DownloadedBytes() const
{
	if (!State.IsSet() || State->IsType<FStateUpdatable>())
	{
		return 0;
	}
	if (const FStateUpdating* StateUpdating = State->TryGet<FStateUpdating>())
	{
		return static_cast<uint64>(FMath::RoundToInt64(StateUpdating->DownloadProgress * StateUpdating->DownloadSize));
	}
	if (const FStateInstalling* StateInstalling = State->TryGet<FStateInstalling>())
	{
		return StateInstalling->DownloadedSize;
	}
	if (const FStateUpdated* StateUpdated = State->TryGet<FStateUpdated>())
	{
		return StateUpdated->DownloadedSize;
	}
	checkf(false, TEXT("Unsupported state type in DownloadedBytes()"));
	return 0;
}

float FInstallBundleReport::InstallationProgress() const
{
	if (!State.IsSet() || State->IsType<FStateUpdatable>() || State->IsType<FStateUpdating>())
	{
		return 0.f;
	}
	if (const FStateInstalling* StateInstalling = State->TryGet<FStateInstalling>())
	{
		check(StateInstalling->InstallProgress >= 0 && StateInstalling->InstallProgress <= 1.f);
		return StateInstalling->InstallProgress;
	}
	if (State->IsType<FStateUpdated>())
	{
		return 1.f;
	}
	checkf(false, TEXT("Unsupported state type in DownloadedBytes()"));
	return 0;
}

void FInstallBundleReport::AddDiscrepancy(uint64 Bytes)
{
	if (!Bytes || !State.IsSet())
	{
		return;
	}
	if (FStateUpdatable* StateUpdatable = State->TryGet<FStateUpdatable>())
	{
		StateUpdatable->DownloadSize += Bytes;
	}
	else if (FStateUpdating* StateUpdating = State->TryGet<FStateUpdating>())
	{
		const uint64 DownloadedBytes = StateUpdating->DownloadProgress * StateUpdating->DownloadSize + Bytes;
		StateUpdating->DownloadSize += Bytes;
		StateUpdating->DownloadProgress = DownloadedBytes / float(StateUpdating->DownloadSize);
	}
	else if (FStateInstalling* StateInstalling = State->TryGet<FStateInstalling>())
	{
		StateInstalling->DownloadedSize += Bytes;
	}
	else if (FStateUpdated* StateInstalled = State->TryGet<FStateUpdated>())
	{
		StateInstalled->DownloadedSize += Bytes;
	}
	else
	{
		checkf(false, TEXT("Unsupported state type in DownloadedBytes()"));
	}
}

FInstallBundleReport* FInstallBundleSourceReport::TryFindBundleReport(FName BundleName)
{
	return Algo::FindByPredicate(Bundles, [&](const FInstallBundleReport& BundleReport) {
		return BundleReport.BundleName == BundleName;
	});
}

FInstallBundleReport& FInstallBundleSourceReport::FindOrAddBundleReport(FName BundleName)
{
	if (FInstallBundleReport* BundleReport = TryFindBundleReport(BundleName))
	{
		return *BundleReport;
	}

	FInstallBundleReport& NewReport = Bundles.Emplace_GetRef();
	NewReport.BundleName = BundleName;
	return *TryFindBundleReport(BundleName);
}

uint64 FInstallBundleSourceReport::TotalDownloadSize() const
{
	uint64 Total = 0;
	for (const FInstallBundleReport& BundleReport : Bundles)
	{
		Total += BundleReport.TotalDownloadSize();
	}
	return Total;
}

uint64 FInstallBundleSourceReport::DownloadedBytes() const
{
	uint64 Total = 0;
	for (const FInstallBundleReport& BundleReport : Bundles)
	{
		Total += BundleReport.DownloadedBytes();
	}
	return Total;
}

int FInstallBundleSourceReport::TotalBundleCount() const
{
	return Bundles.Num();
}

int FInstallBundleSourceReport::UpdatedBundleCount() const
{
	int Num = 0;
	for (const FInstallBundleReport& BundleReport : Bundles)
	{
		if (BundleReport.IsUpdated())
		{
			++Num;
		}
	}
	return Num;
}

FInstallBundleSourceReport* FInstallManagerBundleReport::TryFindBundleSourceReport(const FInstallBundleSourceType& Type)
{
	return Algo::FindByPredicate(SourceReports, [SourceType = FName(Type.GetNameStr())](const FInstallBundleSourceReport& BundleSourceReport) {
		return BundleSourceReport.SourceType == SourceType;
	});
}

FInstallBundleSourceReport& FInstallManagerBundleReport::FindOrAddBundleSourceReport(const FInstallBundleSourceType& Type)
{
	if (FInstallBundleSourceReport* BundleSourceReport = TryFindBundleSourceReport(Type))
	{
		return *BundleSourceReport;
	}

	FInstallBundleSourceReport& NewReport = SourceReports.Emplace_GetRef();
	NewReport.SourceType = FName(Type.GetNameStr());
	return *TryFindBundleSourceReport(Type);
}

float FInstallManagerBundleReport::GetDownloadProgress() const
{
	const uint64 TotalBytes = TotalDownloadSize();
	if (!TotalBytes)
	{
		return 0.f;
	}

	const uint64 Downloaded = DownloadedBytes();
	return float(Downloaded) / float(TotalBytes);
}

float FInstallManagerBundleReport::GetInstallationProgress() const
{
	float TotalProgress = 0;
	float TotalWeight = 0;
	for (const FInstallBundleSourceReport& SourceReport : SourceReports)
	{
		for (const FInstallBundleReport& BundleReport : SourceReport.Bundles)
		{
			const float BundleWeight = BundleReport.TotalDownloadSize() / (1024.f * 1024.f * 1024.f);
			const float BundleProgress = BundleReport.InstallationProgress();
			TotalWeight += BundleWeight;
			TotalProgress += BundleProgress * BundleWeight;
		}
	}
	if (!TotalWeight)
	{
		return 0.f;
	}
	return TotalProgress / TotalWeight;
}

uint64 FInstallManagerBundleReport::TotalDownloadSize() const
{
	uint64 Total = 0;
	for (const FInstallBundleSourceReport& SourceReport : SourceReports)
	{
		Total += SourceReport.TotalDownloadSize();
	}
	return Total;
}

uint64 FInstallManagerBundleReport::DownloadedBytes() const
{
	uint64 Total = 0;
	for (const FInstallBundleSourceReport& SourceReport : SourceReports)
	{
		Total += SourceReport.DownloadedBytes();
	}
	return Total;
}

int FInstallManagerBundleReport::TotalBundleCount() const
{
	int Num = 0;
	for (const FInstallBundleSourceReport& SourceReport : SourceReports)
	{
		Num += SourceReport.TotalBundleCount();
	}
	return Num;
}

int FInstallManagerBundleReport::UpdatedBundleCount() const
{
	int Num = 0;
	for (const FInstallBundleSourceReport& SourceReport : SourceReports)
	{
		Num += SourceReport.UpdatedBundleCount();
	}
	return Num;
}

FString FInstallManagerBundleReport::GetProgressString() const
{
	const uint64 BytesDownloaded = DownloadedBytes();
	const uint64 BytesTotal = TotalDownloadSize();
	const int BundlesUpdated = UpdatedBundleCount();
	const int BundlesTotal = TotalBundleCount();
	const double Progress = BytesTotal ? double(BytesDownloaded) / BytesTotal : 0.0;
	return FString::Printf(TEXT("{Sess.%d} INST %.2f%% DL %.2f%% [%.2f / %.2fGB] [%d / %d bundles]"),
		LoadedCount,
		100.f * GetInstallationProgress(),
		100.f * Progress,
		BytesDownloaded / (1000.f * 1000.f * 1000.f),
		BytesTotal / (1000.f * 1000.f * 1000.f),
		BundlesUpdated,
		BundlesTotal);
}

FString FInstallManagerBundleReport::GetLongProgressString() const
{
	TStringBuilder<512> ProgressString;
	ProgressString.Append(GetProgressString());
	for (int i = 0; i < SourceReports.Num(); ++i)
	{
		if (SourceReports[i].Bundles.IsEmpty())
		{
			continue;
		}

		ProgressString.Appendf(TEXT(" [%d"), i);
		for (int j = 0; j < SourceReports[i].Bundles.Num(); ++j)
		{
			ProgressString.Appendf(TEXT(" #%d,%s,%" UINT64_FMT "/%" UINT64_FMT "M"),
				SourceReports[i].Bundles[j].State->GetIndex(),
				*SourceReports[i].Bundles[j].BundleName.ToString().Right(7),
				SourceReports[i].Bundles[j].DownloadedBytes() / (1000 * 1000),
				SourceReports[i].Bundles[j].TotalDownloadSize() / (1000 * 1000));
		}
		ProgressString.Appendf(TEXT("]"));
	}
	return ProgressString.ToString();
}

FInstallManagerBundleReport::ECombinedStatus FInstallManagerBundleReport::GetCombinedStatus() const
{
	// If no sources or bundles, unknown
	const int BundleCount = TotalBundleCount();
	if (BundleCount == 0)
	{
		return ECombinedStatus::Unknown;
	}

	// Count numbers, if we find at least one unknown, we always return unknown, otherwise we look at the counters afterwards
	int NumUpdatable = 0;
	int NumUpdating = 0;
	int NumInstalling = 0;
	int NumUpdated = 0;
	for (const FInstallBundleSourceReport& SourceReport : SourceReports)
	{
		for (const FInstallBundleReport& BundleReport : SourceReport.Bundles)
		{
			if (BundleReport.IsUnknown())
			{
				return ECombinedStatus::Unknown;
			}
			if (BundleReport.IsUpdatable())
			{
				++NumUpdatable;
			}
			else if (BundleReport.IsUpdating())
			{
				++NumUpdating;
			}
			else if (BundleReport.IsInstalling())
			{
				++NumInstalling;
			}
			else
			{
				++NumUpdated;
			}
		}
	}

	// Sanity check
	check(BundleCount == NumUpdatable + NumUpdating + NumInstalling + NumUpdated);

	// Determine state
	if (NumUpdatable > 0 && NumUpdating == 0 && NumInstalling == 0 && NumUpdated == 0)
	{
		return ECombinedStatus::Updatable;
	}
	if (NumInstalling > 0 && NumUpdating == 0 && NumUpdatable == 0)
	{
		return ECombinedStatus::Finishing;
	}
	if (NumUpdated == BundleCount)
	{
		return ECombinedStatus::Finished;
	}
	return ECombinedStatus::Updating;
}

TOptional<FString> FInstallManagerBundleReport::GetLowestCurrentBundleVersion() const
{
	TOptional<FString> Lowest;
	for (const FInstallBundleSourceReport& SourceReport : SourceReports)
	{
		for (const FInstallBundleReport& BundleReport : SourceReport.Bundles)
		{
			if (BundleReport.SourceVersion.IsSet() && !BundleReport.SourceVersion.GetValue().IsEmpty())
			{
				if (!Lowest.IsSet() || BundleReport.SourceVersion.GetValue() < Lowest.GetValue())
				{
					Lowest = BundleReport.SourceVersion.GetValue();
				}
			}
		}
	}
	return Lowest;
}

TOptional<FString> FInstallManagerBundleReport::GetHighestCurrentBundleVersion() const
{
	TOptional<FString> Highest;
	for (const FInstallBundleSourceReport& SourceReport : SourceReports)
	{
		for (const FInstallBundleReport& BundleReport : SourceReport.Bundles)
		{
			if (BundleReport.SourceVersion.IsSet() && !BundleReport.SourceVersion.GetValue().IsEmpty())
			{
				if (!Highest.IsSet() || BundleReport.SourceVersion.GetValue() > Highest.GetValue())
				{
					Highest = BundleReport.SourceVersion.GetValue();
				}
			}
		}
	}
	return Highest;
}

FInstallManagerBundleReportCache::FInstallManagerBundleReportCache(TArray<TUniquePtr<InstallBundleUtil::FInstallBundleTask>>& AsyncTasks)
	: AsyncTasks(AsyncTasks)
	, ReportFilename(FPaths::Combine(FPlatformMisc::GamePersistentDownloadDir(), TEXT("InstallBundleManagerReportCache.json"))) {
}

void FInstallManagerBundleReportCache::Load(FInstallManagerBundleReportCacheDoneLoadingDelegate OnLoadCompleteCallback)
{
	TSharedPtr<FInstallManagerBundleReport> NewReport = MakeShared<FInstallManagerBundleReport>();
	InstallBundleUtil::StartInstallBundleAsyncIOTask(AsyncTasks,
		[this, NewReport]()
		{
			if (FPaths::FileExists(ReportFilename))
			{
				FString JSONStringOnDisk;
				if (FFileHelper::LoadFileToString(JSONStringOnDisk, *ReportFilename))
				{
					UE_LOGF(LogInstallBundleManager, Display, "Reading InstallManagerBundleReportCache: %ls", *ReportFilename);
					const bool bSuccess = NewReport->FromJson(JSONStringOnDisk);
					ensureAlwaysMsgf(bSuccess, TEXT("Invalid JSON found while parsing InstallManagerBundleReportCache: %s"), *ReportFilename);
					++NewReport->LoadedCount;
				}
				else
				{
					UE_LOGF(LogInstallBundleManager, Warning, "Failed to read: %ls", *ReportFilename);
				}
			}
		},
		[this, NewReport, OnLoadCompleteCallback = MoveTemp(OnLoadCompleteCallback)]()
		{
			Report = MoveTemp(*NewReport);
			OnLoadCompleteCallback.ExecuteIfBound();
		}
	);
}

void FInstallManagerBundleReportCache::SetReport(FInstallManagerBundleReport NewReport)
{
	Report = MoveTemp(NewReport);

	InstallBundleUtil::StartInstallBundleAsyncIOTask(AsyncTasks,
		[this, TargetReport = Report]()
		{
			const bool bSuccess = FFileHelper::SaveStringToFile(TargetReport.ToJson(), *ReportFilename);
			return ensureAlwaysMsgf(bSuccess, TEXT("Error saving JSON output for InstallManagerBundleReportCache: %s"), *ReportFilename);
		},
		[] {});
}
