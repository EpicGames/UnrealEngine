// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleTypes.h"

#include "Containers/ContainerAllocationPolicies.h"
#include "InstallBundleManagerPrivate.h"
#include "InstallBundleUtils.h"
#include "InstallBundleManagerInterface.h"
#include "IO/IoStoreOnDemand.h"
#include "Misc/CString.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const TCHAR* LexToString(EInstallBundleSourceType Type)
{
	static const TCHAR* Strings[] =
	{
		TEXT("Bulk"),
		TEXT("Launcher"),
		TEXT("BuildPatchServices"),
#if WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
		TEXT("Platform"),
#endif // WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
		TEXT("GameCustom"),
		TEXT("Streaming"),
	};

	// Clang has issues with the not silencing deprecation warnings for TLexToString here,
	// so explicitly pass the types to make it happy.
	return InstallBundleUtil::TLexToString<EInstallBundleSourceType, decltype(Strings), EInstallBundleSourceType::Count>(Type, Strings);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void LexFromString(EInstallBundleSourceType& OutType, const TCHAR* String)
{
	OutType = EInstallBundleSourceType::Count;

	for (EInstallBundleSourceType SourceType : TEnumRange<EInstallBundleSourceType>())
	{
		const TCHAR* SourceStr = LexToString(SourceType);
		if (FCString::Stricmp(SourceStr, String) == 0)
		{
			OutType = SourceType;
			break;
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class FInstallBundleSourceTypeNameTable
{
public:
	FInstallBundleSourceTypeNameTable()
	{
		check(IsInGameThread());

		// Find all possible sources from config
		TArray<FString> ConfgSources;
		TMap<FString, FString> ConfigFallbackSources;
		if (!InstallBundleUtil::GetConfiguredBundleSources(ConfgSources, ConfigFallbackSources))
		{
			return;
		}

		for (FString& Source : ConfgSources)
		{
			NameTable.AddUnique(MoveTemp(Source));
		}

		for (TPair<FString, FString>& Pair : ConfigFallbackSources)
		{
			NameTable.AddUnique(MoveTemp(Pair.Key));
			NameTable.AddUnique(MoveTemp(Pair.Value));
		}
	}

	FStringView FindBundleSourceTypeByName(FStringView InName) const
	{
		for (const FString& Str : NameTable)
		{
			if (Str == InName)
			{
				return Str;
			}
		}

		return FStringView(TEXTVIEW(""));
	}

private:
	TArray<FString, TInlineAllocator<8>> NameTable;
};

static const FInstallBundleSourceTypeNameTable& GetInstallBundleSourceTypeNameTable()
{
	static FInstallBundleSourceTypeNameTable InstallBundleSourceTypeNameTable;
	return InstallBundleSourceTypeNameTable;
}

FInstallBundleSourceType::FInstallBundleSourceType(FStringView InNameStr)
	: NameStr(GetInstallBundleSourceTypeNameTable().FindBundleSourceTypeByName(InNameStr))
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FInstallBundleSourceType::FInstallBundleSourceType(EInstallBundleSourceType InLegacySourceType)
	: NameStr(GetInstallBundleSourceTypeNameTable().FindBundleSourceTypeByName(LexToString(InLegacySourceType)))
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const TCHAR* LexToString(FInstallBundleSourceType Type)
{
	return Type.GetNameCStr();
}

const TCHAR* LexToString(EInstallBundleManagerInitResult Result)
{
	static const TCHAR* Strings[] =
	{
		TEXT("OK"),
		TEXT("BuildMetaDataNotFound"),
		TEXT("RemoteBuildMetaDataNotFound"),
		TEXT("BuildMetaDataDownloadError"),
		TEXT("BuildMetaDataParsingError"),
		TEXT("DistributionRootParseError"),
		TEXT("DistributionRootDownloadError"),
		TEXT("ManifestArchiveError"),
		TEXT("ManifestCreationError"),
		TEXT("ManifestDownloadError"),
		TEXT("BackgroundDownloadsIniDownloadError"),
		TEXT("NoInternetConnectionError"),
		TEXT("ConfigurationError"),
		TEXT("ClientPatchRequiredError"),
	};

	return InstallBundleUtil::TLexToString(Result, Strings);
}

const TCHAR* LexToString(EInstallBundleInstallState State)
{
	static const TCHAR* Strings[] =
	{
		TEXT("NotInstalled"),
		TEXT("NeedsUpdate"),
		TEXT("UpToDate"),
	};

	return InstallBundleUtil::TLexToString(State, Strings);
}

const TCHAR* LexToString(EInstallBundleResult Result)
{
	static const TCHAR* Strings[] =
	{
		TEXT("OK"),
		TEXT("FailedPrereqRequiresLatestClient"),
		TEXT("FailedPrereqRequiresLatestContent"),
		TEXT("FailedCacheReserve"),
		TEXT("InstallError"),
		TEXT("InstallerOutOfDiskSpaceError"),
		TEXT("ManifestArchiveError"),
		TEXT("ConnectivityError"),
		TEXT("UserCancelledError"),
		TEXT("InitializationError"),
		TEXT("InitializationPending"),
		TEXT("MetadataError"),
	};

	return InstallBundleUtil::TLexToString(Result, Strings);
}

const TCHAR* LexToString(EInstallBundleReleaseResult Result)
{
	static const TCHAR* Strings[] =
	{
		TEXT("OK"),
		TEXT("ManifestArchiveError"),
		TEXT("UserCancelledError"),
		TEXT("MetadataError"),
	};

	return InstallBundleUtil::TLexToString(Result, Strings);
}

const TCHAR* LexToString(EInstallBundleStatus Status)
{
	static const TCHAR* Strings[] =
	{
		TEXT("Requested"),
		TEXT("Updating"),
		TEXT("Finishing"),
		TEXT("Ready"),
	};

	return InstallBundleUtil::TLexToString(Status, Strings);
}

const TCHAR* LexToString(EInstallBundleManagerPatchCheckResult EnumVal)
{
	// These are namespaced because PartyHub expects them that way :/
	static const TCHAR* Strings[] =
	{
		TEXT("EInstallBundleManagerPatchCheckResult::NoPatchRequired"),
		TEXT("EInstallBundleManagerPatchCheckResult::ClientPatchRequired"),
		TEXT("EInstallBundleManagerPatchCheckResult::ContentPatchRequired"),
		TEXT("EInstallBundleManagerPatchCheckResult::NoLoggedInUser"),
		TEXT("EInstallBundleManagerPatchCheckResult::PatchCheckFailure"),
	};

	return InstallBundleUtil::TLexToString(EnumVal, Strings);
}

const TCHAR* LexToString(EInstallBundlePriority Priority)
{
	static const TCHAR* Strings[] =
	{
		TEXT("High"),
		TEXT("Normal"),
		TEXT("Low"),
	};

	return InstallBundleUtil::TLexToString(Priority, Strings);
}

const TCHAR* LexToString(EInstallBundleSourceUpdateBundleInfoResult Result)
{
	static const TCHAR* Strings[] =
	{
		TEXT("OK"),
		TEXT("NotInitailized"),
		TEXT("AlreadyMounted"),
		TEXT("AlreadyRequested"),
		TEXT("IllegalCacheStatus"),
	};

	return InstallBundleUtil::TLexToString(Result, Strings);
}

bool LexTryParseString(EInstallBundlePriority& OutMode, const TCHAR* InBuffer)
{
	if (FCString::Stricmp(InBuffer, TEXT("High")) == 0)
	{
		OutMode = EInstallBundlePriority::High;
		return true;
	}
	if (FCString::Stricmp(InBuffer, TEXT("Normal")) == 0)
	{
		OutMode = EInstallBundlePriority::Normal;
		return true;
	}
	if (FCString::Stricmp(InBuffer, TEXT("Low")) == 0)
	{
		OutMode = EInstallBundlePriority::Low;
		return true;
	}
	return false;
}

bool FInstallBundleCombinedInstallState::GetAllBundlesHaveState(EInstallBundleInstallState State, TArrayView<const FName> ExcludedBundles) const
{
	for (const TPair<FName, EInstallBundleInstallState>& Pair : IndividualBundleStates)
	{
		if (ExcludedBundles.Contains(Pair.Key))
			continue;

		if (Pair.Value != State)
			return false;
	}

	return true;
}

bool FInstallBundleCombinedInstallState::GetAnyBundleHasState(EInstallBundleInstallState State, TArrayView<const FName> ExcludedBundles) const
{
	for (const TPair<FName, EInstallBundleInstallState>& Pair : IndividualBundleStates)
	{
		if (ExcludedBundles.Contains(Pair.Key))
			continue;

		if (Pair.Value == State)
			return true;
	}

	return false;
}

bool FInstallBundleCombinedContentState::GetAllBundlesHaveState(EInstallBundleInstallState State, TArrayView<const FName> ExcludedBundles /*= TArrayView<const FName>()*/) const
{
	for (const TPair<FName, FInstallBundleContentState>& Pair : IndividualBundleStates)
	{
		if (ExcludedBundles.Contains(Pair.Key))
			continue;

		if (Pair.Value.State != State)
			return false;
	}

	return true;
}

bool FInstallBundleCombinedContentState::GetAnyBundleHasState(EInstallBundleInstallState State, TArrayView<const FName> ExcludedBundles /*= TArrayView<const FName>()*/) const
{
	for (const TPair<FName, FInstallBundleContentState>& Pair : IndividualBundleStates)
	{
		if (ExcludedBundles.Contains(Pair.Key))
			continue;

		if (Pair.Value.State == State)
			return true;
	}

	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FInstallBundleSourceUpdateContentResultInfo::FInstallBundleSourceUpdateContentResultInfo() = default;
FInstallBundleSourceUpdateContentResultInfo::FInstallBundleSourceUpdateContentResultInfo(FInstallBundleSourceUpdateContentResultInfo&&) = default;
FInstallBundleSourceUpdateContentResultInfo::~FInstallBundleSourceUpdateContentResultInfo() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
