// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstallBundleUtils.h"

#include "InstallBundleManagerPrivate.h"
#include "Misc/App.h"

#include "HAL/PlatformApplicationMisc.h"

#include "Containers/Ticker.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformFileManager.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Stats/Stats.h"

#include "Algo/AnyOf.h"
#include "Algo/AllOf.h"
#include "Algo/Find.h"

namespace InstallBundleUtil
{
	FString GetAppVersion()
	{
		return FString::Printf(TEXT("%s-%s"), FApp::GetBuildVersion(), ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()));
	}

	bool HasInternetConnection(ENetworkConnectionType ConnectionType)
	{
		return ConnectionType != ENetworkConnectionType::AirplaneMode
			&& ConnectionType != ENetworkConnectionType::None;
	}

	bool SplitHostUrl(const FStringView Url, FStringView& OutHost, FStringView& OutRemainder)
	{
		OutHost = OutRemainder = FStringView();

		const FStringView ProtoHttp = TEXTVIEW("http://");
		const FStringView ProtoHttps = TEXTVIEW("https://");
		const FStringView ProtoFile = TEXTVIEW("file://");

		FStringView Proto;
		if (Url.StartsWith(ProtoHttp))
		{
			Proto = ProtoHttp;
		}
		else if (Url.StartsWith(ProtoHttps))
		{
			Proto = ProtoHttps;
		}
		else if (Url.StartsWith(ProtoFile))
		{
			Proto = ProtoFile;
		}
		else
		{
			return false;
		}

		FStringView ProtoChopped = Url.RightChop(Proto.Len());

		int32 DelimPos = INDEX_NONE;
		if (!ProtoChopped.FindChar(TEXT('/'), DelimPos))
		{
			OutHost = ProtoChopped;
			return !OutHost.IsEmpty();
		}

		OutHost = Url.Left(Proto.Len() + DelimPos);
		OutRemainder = Url.RightChop(OutHost.Len());

		return true;
	}

	const TCHAR* GetInstallBundlePauseReason(EInstallBundlePauseFlags Flags)
	{
		// Return the most appropriate reason given the flags

		if (EnumHasAnyFlags(Flags, EInstallBundlePauseFlags::UserPaused))
			return TEXT("UserPaused");

		if (EnumHasAnyFlags(Flags, EInstallBundlePauseFlags::NoInternetConnection))
			return TEXT("NoInternetConnection");

		if (EnumHasAnyFlags(Flags, EInstallBundlePauseFlags::OnCellularNetwork))
			return TEXT("OnCellularNetwork");

		if (EnumHasAnyFlags(Flags, EInstallBundlePauseFlags::OnLimitedDataNetwork))
			return TEXT("OnLimitedDataNetwork");

		return TEXT("");
	}

	const FString& GetInstallBundleSectionPrefix()
	{
		static FString Prefix("InstallBundleDefinition "); // trailing space intentional
		return Prefix;
	}

	const FString& GetInstallBundleSubsetSectionPrefix()
	{
		static FString Prefix("InstallBundleSubsetDefinition "); // trailing space intentional
		return Prefix;
	}

	bool GetConfiguredBundleSources(TArray<FString>& OutSources, TMap<FString, FString>& OutFallbackSources)
	{
#ifdef INSTALL_BUNDLE_SOURCES_FALLBACK_CONFIG_SECTION
		const TCHAR* BundleSourceFallbackSection = TEXT(INSTALL_BUNDLE_SOURCES_FALLBACK_CONFIG_SECTION);
#else
		const TCHAR* BundleSourceFallbackSection = TEXT("InstallBundleManager.FallbackBundleSources");
#endif // INSTALL_BUNDLE_SOURCES_FALLBACK_CONFIG_SECTION

		TArray<FString> ConfigBundleFallbacks;
		if (GConfig->GetArray(BundleSourceFallbackSection, TEXT("FallbackBundleSources"), ConfigBundleFallbacks, GInstallBundleIni))
		{
			OutFallbackSources.Empty(ConfigBundleFallbacks.Num());
			for (FString& ConfigFallback : ConfigBundleFallbacks)
			{
				// Remove parentheses
				ConfigFallback.ReplaceInline(TEXT("("), TEXT(""));
				ConfigFallback.ReplaceInline(TEXT(")"), TEXT(""));

				TArray<FString> Tokens;
				if (2 != ConfigFallback.ParseIntoArrayWS(Tokens, TEXT(",")))
				{
					ensureAlwaysMsgf(false, TEXT("Malformed entry in InstallBundleManager.FallbackBundleSources"));
					return false;
				}

				OutFallbackSources.Add(MoveTemp(Tokens[0]), MoveTemp(Tokens[1]));
			}
		}

#ifdef INSTALL_BUNDLE_SOURCES_CONFIG_SECTION
		const TCHAR* BundleSourceSection = TEXT(INSTALL_BUNDLE_SOURCES_CONFIG_SECTION);
#else
		const TCHAR* BundleSourceSection = TEXT("InstallBundleManager.BundleSources");
#endif // INSTALL_BUNDLE_SOURCE_CONFIG_SECTION

		TArray<FString> ConfigBundleSources;
		if (ensureAlways(GConfig->GetArray(BundleSourceSection, TEXT("DefaultBundleSources"), ConfigBundleSources, GInstallBundleIni)))
		{
			OutSources = MoveTemp(ConfigBundleSources);
		}
		else
		{
			return false;
		}

		return true;
	}

	bool HasInstallBundleInConfig(const FString& BundleName)
	{
		const FConfigFile* InstallBundleConfig = GConfig->FindConfigFile(GInstallBundleIni);
		if (InstallBundleConfig)
		{
			const FString SectionName = InstallBundleUtil::GetInstallBundleSectionPrefix() + BundleName;
			return InstallBundleConfig->DoesSectionExist(*SectionName);
		}
		return false;
	}

	bool GetMountOptionsFromConfig(const FStringView BundleName, FConfigMountOptions& OutMountOptions)
	{
		const FConfigFile* InstallBundleConfig = GConfig->FindConfigFile(GInstallBundleIni);
		if (InstallBundleConfig)
		{
			const FString SectionName = InstallBundleUtil::GetInstallBundleSectionPrefix() + BundleName;
			if (InstallBundleConfig->DoesSectionExist(*SectionName))
			{
				OutMountOptions.bWithSoftReferences = false;
				InstallBundleConfig->GetBool(*SectionName, TEXT("bMountSoftReferences"), OutMountOptions.bWithSoftReferences);
				return true;
			}
		}
		return false;
	}

	bool AllInstallBundlePredicate(const FConfigFile& InstallBundleConfig, const FString& Section)
	{ 
		return true; 
	}

	bool IsPlatformInstallBundlePredicate(const FConfigFile& InstallBundleConfig, const FString& Section)
	{
		FString PlatformChunkName;
		InstallBundleConfig.GetString(*Section, TEXT("PlatformChunkName"), PlatformChunkName);

		int32 ChunkID = 0;
		if (PlatformChunkName.IsEmpty() && InstallBundleConfig.GetInt(*Section, TEXT("PlatformChunkID"), ChunkID) && ChunkID < 0)
		{
			return false;
		}

		return true;
	}

	TArray<TPair<FString, TArray<FRegexPattern>>> LoadBundleRegexFromConfig(
		const FConfigFile& InstallBundleConfig, 
		TFunctionRef<bool(const FConfigFile& InstallBundleConfig, const FString& Section)> SectionPredicate /*= AllInstallBundlePredicate*/)
	{
		TArray<TPair<FString, TArray<FRegexPattern>>> BundleRegexList; // BundleName -> FileRegex

		for (const TPair<FString, FConfigSection>& Pair : InstallBundleConfig)
		{
			const FString& Section = Pair.Key;
			if (!Section.StartsWith(InstallBundleUtil::GetInstallBundleSectionPrefix()))
				continue;

			if (!SectionPredicate(InstallBundleConfig, Section))
				continue;

			TArray<FString> StrSearchRegexPatterns;
			if (!InstallBundleConfig.GetArray(*Section, TEXT("FileRegex"), StrSearchRegexPatterns))
				continue;

			TArray<FRegexPattern> SearchRegexPatterns;
			SearchRegexPatterns.Reserve(StrSearchRegexPatterns.Num());
			for (const FString& Str : StrSearchRegexPatterns)
			{
				SearchRegexPatterns.Emplace(Str, ERegexPatternFlags::CaseInsensitive);
			}

			const FString BundleName = Section.RightChop(InstallBundleUtil::GetInstallBundleSectionPrefix().Len());
			BundleRegexList.Emplace(TPair<FString, TArray<FRegexPattern>>(BundleName, MoveTemp(SearchRegexPatterns)));
		}

		BundleRegexList.StableSort([&InstallBundleConfig](const TPair<FString, TArray<FRegexPattern>>& PairA, const TPair<FString, TArray<FRegexPattern>>& PairB) -> bool
		{
			int32 BundleAOrder = INT_MAX;
			int32 BundleBOrder = INT_MAX;

			const FString SectionA = InstallBundleUtil::GetInstallBundleSectionPrefix() + PairA.Key;
			const FString SectionB = InstallBundleUtil::GetInstallBundleSectionPrefix() + PairB.Key;

			if (!InstallBundleConfig.GetInt(*SectionA, TEXT("Order"), BundleAOrder))
			{
				UE_LOGF(LogInstallBundleManager, Warning, "Bundle Section %ls doesn't have an order", *SectionA);
			}

			if (!InstallBundleConfig.GetInt(*SectionB, TEXT("Order"), BundleBOrder))
			{
				UE_LOGF(LogInstallBundleManager, Warning, "Bundle Section %ls doesn't have an order", *SectionB);
			}

			return BundleAOrder < BundleBOrder;
		});

		return BundleRegexList;
	}

	TArray<TPair<FString, TArray<FRegexPattern>>> LoadBundleSubsetRegexFromConfig(const FConfigFile& InstallBundleConfig)
	{
		TArray<TPair<FString, TArray<FRegexPattern>>> BundleSubsetRegexList; // BundleSubsetName -> FileRegex

		for (const TPair<FString, FConfigSection>& Pair : InstallBundleConfig)
		{
			const FString& Section = Pair.Key;
			if (!Section.StartsWith(InstallBundleUtil::GetInstallBundleSubsetSectionPrefix()))
			{
				continue;
			}

			TArray<FString> StrSearchRegexPatterns;
			InstallBundleConfig.GetArray(*Section, TEXT("FileRegex"), StrSearchRegexPatterns);

			TArray<FRegexPattern> SearchRegexPatterns;
			SearchRegexPatterns.Reserve(StrSearchRegexPatterns.Num());
			for (const FString& Str : StrSearchRegexPatterns)
			{
				SearchRegexPatterns.Emplace(Str, ERegexPatternFlags::CaseInsensitive);
			}

			const FString BundleSubsetName = Section.RightChop(InstallBundleUtil::GetInstallBundleSubsetSectionPrefix().Len());
			BundleSubsetRegexList.Emplace(TPair<FString, TArray<FRegexPattern>>(BundleSubsetName, MoveTemp(SearchRegexPatterns)));
		}

		return BundleSubsetRegexList;
	}

	bool MatchBundleRegex(
		const TArray<TPair<FString, TArray<FRegexPattern>>>& BundleRegexList,
		const FString& Path,
		FString& OutBundleName)
	{
		const TPair<FString, TArray<FRegexPattern>>* BundleRegexPair = Algo::FindByPredicate(BundleRegexList,
			[&Path](const TPair<FString, TArray<FRegexPattern>>& Pair)
			{
				const TArray<FRegexPattern>& SearchRegexPatterns = Pair.Value;
				return Algo::AnyOf(SearchRegexPatterns, [&Path](const FRegexPattern& Pattern)
				{
					return FRegexMatcher(Pattern, Path).FindNext();
				});
			});

		if (BundleRegexPair)
		{
			OutBundleName = BundleRegexPair->Key;
			return true;
		}

		return false;
	}

	bool MatchBundleSubsetRegex(
		const FString& BundleSubset,
		const TArray<TPair<FString, TArray<FRegexPattern>>>& BundleSubsetRegexList,
		const FString& Path)
	{
		const TPair<FString, TArray<FRegexPattern>>* BundleRegexPair = Algo::FindByPredicate(BundleSubsetRegexList,
			[&BundleSubset, &Path](const TPair<FString, TArray<FRegexPattern>>& Pair)
			{
				if (Pair.Key != BundleSubset)
				{
					return false;
				}

				const TArray<FRegexPattern>& SearchRegexPatterns = Pair.Value;
				return Algo::AnyOf(SearchRegexPatterns, [&Path](const FRegexPattern& Pattern)
				{
					return FRegexMatcher(Pattern, Path).FindNext();
				});
			});

		return !!BundleRegexPair;
	}

	FName FInstallBundleManagerKeepAwake::Tag(TEXT("InstallBundleManagerKeepAwake"));
	FName FInstallBundleManagerKeepAwake::TagWithRendering(TEXT("InstallBundleManagerKeepAwakeWithRendering"));

	bool FInstallBundleManagerScreenSaverControl::bDidDisableScreensaver = false;
	int FInstallBundleManagerScreenSaverControl::DisableCount = 0;

	void FInstallBundleManagerScreenSaverControl::IncDisable()
	{
		if (!bDidDisableScreensaver && FPlatformApplicationMisc::IsScreensaverEnabled())
		{
			bDidDisableScreensaver = FPlatformApplicationMisc::ControlScreensaver(FGenericPlatformApplicationMisc::EScreenSaverAction::Disable);
		}

		++DisableCount;
	}

	void FInstallBundleManagerScreenSaverControl::DecDisable()
	{
		--DisableCount;
		if (DisableCount == 0 && bDidDisableScreensaver)
		{
			FPlatformApplicationMisc::ControlScreensaver(FGenericPlatformApplicationMisc::EScreenSaverAction::Enable);
			bDidDisableScreensaver = false;
		}
	}

	std::atomic<int32> InstallBundleSuppressAnalyticsCounter = 0;

	FInstallBundleSuppressAnalytics::FInstallBundleSuppressAnalytics()
		: bIsEnabled(false)
	{
	}

	FInstallBundleSuppressAnalytics::~FInstallBundleSuppressAnalytics()
	{
		Disable();
	}

	void FInstallBundleSuppressAnalytics::Enable()
	{
		if (!bIsEnabled)
		{
			bIsEnabled = true;
			ensure(InstallBundleSuppressAnalyticsCounter++ >= 0);
		}
	}

	void FInstallBundleSuppressAnalytics::Disable()
	{
		if (bIsEnabled)
		{
			bIsEnabled = false;
			ensure(--InstallBundleSuppressAnalyticsCounter >= 0);
		}
	}

	bool FInstallBundleSuppressAnalytics::IsEnabled()
	{
		return InstallBundleSuppressAnalyticsCounter > 0;
	}

	void StartInstallBundleAsyncIOTask(TUniqueFunction<void()> WorkFunc)
	{
		return StartInstallBundleAsyncIOTask(GIOThreadPool, MoveTemp(WorkFunc));
	}

	void StartInstallBundleAsyncIOTask(FQueuedThreadPool* ThreadPool, TUniqueFunction<void()> WorkFunc)
	{
		(new FAutoDeleteInstallBundleTask(MoveTemp(WorkFunc), nullptr))->StartBackgroundTask(ThreadPool);
	}

	void StartInstallBundleAsyncIOTask(TArray<TUniquePtr<FInstallBundleTask>>& Tasks, TUniqueFunction<void()> WorkFunc, TUniqueFunction<void()> OnComplete)
	{
		TUniquePtr<FInstallBundleTask> Task = MakeUnique<FInstallBundleTask>(MoveTemp(WorkFunc), MoveTemp(OnComplete));
		Task->StartBackgroundTask(GIOThreadPool);
		Tasks.Add(MoveTemp(Task));
	}

	void StartInstallBundleAsyncIOTask(FQueuedThreadPool* ThreadPool, TArray<TUniquePtr<FInstallBundleTask>>& Tasks, TUniqueFunction<void()> WorkFunc, TUniqueFunction<void()> OnComplete)
	{
		TUniquePtr<FInstallBundleTask> Task = MakeUnique<FInstallBundleTask>(MoveTemp(WorkFunc), MoveTemp(OnComplete));
		Task->StartBackgroundTask(ThreadPool);
		Tasks.Add(MoveTemp(Task));
	}

	void FinishInstallBundleAsyncIOTasks(TArray<TUniquePtr<FInstallBundleTask>>& Tasks)
	{
		TArray<TUniquePtr<FInstallBundleTask>> FinishedTasks;
		for (int32 i = 0; i < Tasks.Num();)
		{
			TUniquePtr<FInstallBundleTask>& Task = Tasks[i];
			check(Task);
			if (Task->IsDone())
			{
				FinishedTasks.Add(MoveTemp(Task));
				Tasks.RemoveAtSwap(i, EAllowShrinking::No);
			}
			else
			{
				++i;
			}
		}
		for (TUniquePtr<FInstallBundleTask>& Task : FinishedTasks)
		{
			Task->GetTask().CallOnComplete();
		}
	}

	void CleanupInstallBundleAsyncIOTasks(TArray<TUniquePtr<FInstallBundleTask>>& Tasks)
	{
		for (TUniquePtr<FInstallBundleTask>& Task : Tasks)
		{
			check(Task);
			if (!Task->Cancel())
			{
				Task->EnsureCompletion(false);
			}
		}
	}

	void FContentRequestStatsMap::StatsBegin(const FContentRequestStatsKey& Key)
	{
		FContentRequestStats& Stats = StatsMap.FindOrAdd(Key);
		if (false == ensureAlwaysMsgf(Stats.bOpen, TEXT("StatsBegin - Stat closed for %s"), *Key.GetBundleName().ToString()))
		{
			Stats = FContentRequestStats();
		}

		Stats.StartTime = FPlatformTime::Seconds();
	}

	void FContentRequestStatsMap::StatsEnd(const FContentRequestStatsKey& Key)
	{
		FContentRequestStats& Stats = StatsMap.FindOrAdd(Key);

		ensureAlwaysMsgf(Stats.bOpen && Stats.StartTime > 0, TEXT("StatsEnd - Stat closed for %s"), *Key.GetBundleName().ToString());
		if (Stats.bOpen)
		{
			ensureAlwaysMsgf(
				Algo::AllOf(Stats.StateStats, 
					[](const TPair<FString, FContentRequestStateStats>& Pair) { return !Pair.Value.bOpen; }),
				TEXT("StatsEnd - StateStat open for %s"), *Key.GetBundleName().ToString());

			Stats.EndTime = FPlatformTime::Seconds();
			Stats.bOpen = false;
		}
	}

	void FContentRequestStatsMap::StatsReset(const FContentRequestStatsKey& Key)
	{
		if (FContentRequestStats* Stats = StatsMap.Find(Key))
		{
			ensureAlwaysMsgf(!Stats->bOpen, TEXT("StatsReset - Stat open for %s"), *Key.GetBundleName().ToString());
			ensureAlwaysMsgf(
				Algo::AllOf(Stats->StateStats,
					[](const TPair<FString, FContentRequestStateStats>& Pair) { return !Pair.Value.bOpen; }),
				TEXT("StatsReset - StateStat open for %s"), *Key.GetBundleName().ToString());

			StatsMap.Remove(Key);
		}
	}

	void FContentRequestStatsMap::StatsBegin(const FContentRequestStatsKey& Key, const TCHAR* State)
	{
		FContentRequestStats& Stats = StatsMap.FindOrAdd(Key);
		if (false == ensureAlwaysMsgf(Stats.bOpen, TEXT("StatsBegin - Stat closed for %s - %s"), *Key.GetBundleName().ToString(), State))
		{
			Stats = FContentRequestStats();
			Stats.StartTime = FPlatformTime::Seconds();
		}

		FContentRequestStateStats& StateStats = Stats.StateStats.FindOrAdd(State);
		if (false == ensureAlwaysMsgf(StateStats.bOpen, TEXT("StatsBegin - StateStat closed for %s - %s"), *Key.GetBundleName().ToString(), State))
		{
			StateStats = FContentRequestStateStats();
		}

		StateStats.StartTime = FPlatformTime::Seconds();
	}

	void FContentRequestStatsMap::StatsEnd(const FContentRequestStatsKey& Key, const TCHAR* State, uint64 DataSize /*= 0*/)
	{
		FContentRequestStats& Stats = StatsMap.FindOrAdd(Key);
		if (false == ensureAlwaysMsgf(Stats.bOpen && Stats.StartTime > 0, TEXT("StatsEnd - Stat closed for %s - %s"), *Key.GetBundleName().ToString(), State))
		{
			Stats = FContentRequestStats();
			Stats.StartTime = FPlatformTime::Seconds();
		}

		FContentRequestStateStats& StateStats = Stats.StateStats.FindOrAdd(State);
		if (ensureAlwaysMsgf(StateStats.bOpen && StateStats.StartTime > 0, TEXT("StatsEnd - StateStat closed for %s - %s"), *Key.GetBundleName().ToString(), State))
		{
			StateStats.EndTime = FPlatformTime::Seconds();
			StateStats.DataSize = DataSize;
			StateStats.bOpen = false;
		}
	}

	namespace PersistentStats
	{
		const FString& LexToString(ETimingStatNames InType)
		{
			static const FString TotalTime_Real("TotalTime_Real");
			static const FString TotalTime_FG("TotalTime_FG");
			static const FString TotalTime_BG("TotalTime_BG");
			static const FString ChunkDBDownloadTime_Real("ChunkDBDownloadTime_Real");
			static const FString ChunkDBDownloadTime_FG("ChunkDBDownloadTime_FG");
			static const FString ChunkDBDownloadTime_BG("ChunkDBDownloadTime_BG");
			static const FString InstallTime_Real("InstallTime_Real");
			static const FString InstallTime_FG("InstallTime_FG");
			static const FString InstallTime_BG("InstallTime_BG");
			static const FString PSOTime_Real("PSOTime_Real");
			static const FString PSOTime_FG("PSOTime_FG");
			static const FString PSOTime_BG("PSOTime_BG");

			static const FString Unknown("<Unknown PersistentStats::ETimingStatNames Value>");

			switch (InType)
			{
			case ETimingStatNames::TotalTime_Real:
				return TotalTime_Real;
			case ETimingStatNames::TotalTime_FG:
				return TotalTime_FG;
			case ETimingStatNames::TotalTime_BG:
				return TotalTime_BG;
			case ETimingStatNames::ChunkDBDownloadTime_Real:
				return ChunkDBDownloadTime_Real;
			case ETimingStatNames::ChunkDBDownloadTime_FG:
				return ChunkDBDownloadTime_FG;
			case ETimingStatNames::ChunkDBDownloadTime_BG:
				return ChunkDBDownloadTime_BG;
			case ETimingStatNames::InstallTime_Real:
				return InstallTime_Real;
			case ETimingStatNames::InstallTime_FG:
				return InstallTime_FG;
			case ETimingStatNames::InstallTime_BG:
				return InstallTime_BG;
			case ETimingStatNames::PSOTime_Real:
				return PSOTime_Real;
			case ETimingStatNames::PSOTime_FG:
				return PSOTime_FG;
			case ETimingStatNames::PSOTime_BG:
				return PSOTime_BG;
			default:
				break;
			}

			ensureAlwaysMsgf(false, TEXT("Missing PersistentStats::ETimingStatNames LexToString entry! Missing Entry as Int: %d"), (int)(InType));
			return Unknown;
		}
	
		const FString& LexToString(ECountStatNames InType)
		{
			static const FString NumResumedFromBackground("NumResumedFromBackground");
			static const FString NumResumedFromLaunch("NumResumedFromLaunch");
			static const FString NumBackgrounded("NumBackgrounded");

			static const FString Unknown("<Unknown PersistentStats::ETimingStatNames Value>");

			switch (InType)
			{
			case ECountStatNames::NumResumedFromBackground:
				return NumResumedFromBackground;
			case ECountStatNames::NumResumedFromLaunch:
				return NumResumedFromLaunch;
			case ECountStatNames::NumBackgrounded:
				return NumBackgrounded;
			default:
				break;
			}

			ensureAlwaysMsgf(false, TEXT("Missing PersistentStats::ECountStatNames LexToString entry! Missing Entry as Int: %d"), (int)(InType));
			return Unknown;
		}

		
		bool IsTimerReal(ETimingStatNames InTimerType)
		{
			switch (InTimerType)
			{
				//Intentional fallthrough for known true types
				case ETimingStatNames::TotalTime_Real:
				case ETimingStatNames::ChunkDBDownloadTime_Real:
				case ETimingStatNames::InstallTime_Real:
				case ETimingStatNames::PSOTime_Real:
					return true;

				//Intentional fallthrough for known false types
				case ETimingStatNames::TotalTime_FG:
				case ETimingStatNames::TotalTime_BG:
				case ETimingStatNames::ChunkDBDownloadTime_FG:
				case ETimingStatNames::ChunkDBDownloadTime_BG:
				case ETimingStatNames::InstallTime_FG:
				case ETimingStatNames::InstallTime_BG:
				case ETimingStatNames::PSOTime_FG:
				case ETimingStatNames::PSOTime_BG:
					return false;
					
				default:
					break;
			}

			ensureAlwaysMsgf(false, TEXT("Missing PersistentStats::ETimingStatNames IsTimerReal entry! Missing Entry:%s"), *LexToString(InTimerType));
			return false;
		}

		bool IsTimerFG(ETimingStatNames InTimerType)
		{
			switch (InTimerType)
			{
				//Intentional fallthrough for known true types
				case ETimingStatNames::TotalTime_FG:
				case ETimingStatNames::ChunkDBDownloadTime_FG:
				case ETimingStatNames::InstallTime_FG:
				case ETimingStatNames::PSOTime_FG:
					return true;

				//Intentional fallthrough for known false types
				case ETimingStatNames::TotalTime_Real:
				case ETimingStatNames::TotalTime_BG:
				case ETimingStatNames::ChunkDBDownloadTime_Real:
				case ETimingStatNames::ChunkDBDownloadTime_BG:
				case ETimingStatNames::InstallTime_Real:
				case ETimingStatNames::InstallTime_BG:
				case ETimingStatNames::PSOTime_Real:
				case ETimingStatNames::PSOTime_BG:
					return false;

				default:
					break;
			}

			ensureAlwaysMsgf(false, TEXT("Missing PersistentStats::ETimingStatNames IsTimerFG entry! Missing Entry as Int: %s"), *LexToString(InTimerType));
			return false;
		}

		bool IsTimerBG(ETimingStatNames InTimerType)
		{
			switch (InTimerType)
			{
				//Intentional fallthrough for known true types
				case ETimingStatNames::TotalTime_BG:
				case ETimingStatNames::ChunkDBDownloadTime_BG:
				case ETimingStatNames::InstallTime_BG:
				case ETimingStatNames::PSOTime_BG:
					return true;

				//Intentional fallthrough for known false types
				case ETimingStatNames::TotalTime_Real:
				case ETimingStatNames::TotalTime_FG:
				case ETimingStatNames::ChunkDBDownloadTime_Real:
				case ETimingStatNames::ChunkDBDownloadTime_FG:
				case ETimingStatNames::InstallTime_Real:
				case ETimingStatNames::InstallTime_FG:
				case ETimingStatNames::PSOTime_Real:
				case ETimingStatNames::PSOTime_FG:
					return false;

				default:
					break;
			}

			ensureAlwaysMsgf(false, TEXT("Missing PersistentStats::ETimingStatNames IsTimerBG entry! Missing Entry as Int: %s"), *LexToString(InTimerType));
			return false;
		}

		ETimingStatNames GetAssociatedRealTimerName(ETimingStatNames InTimerType)
		{
			switch (InTimerType)
			{
				//Intentional fallthrough for TotalTime
				case ETimingStatNames::TotalTime_Real:
				case ETimingStatNames::TotalTime_FG:
				case ETimingStatNames::TotalTime_BG:
					return ETimingStatNames::TotalTime_Real;

				//Intentional fallthrough for ChunkDBDownloadTime
				case ETimingStatNames::ChunkDBDownloadTime_Real:
				case ETimingStatNames::ChunkDBDownloadTime_FG:
				case ETimingStatNames::ChunkDBDownloadTime_BG:
					return ETimingStatNames::ChunkDBDownloadTime_Real;

				//Intentional fallthrough for InstallTime
				case ETimingStatNames::InstallTime_Real:
				case ETimingStatNames::InstallTime_FG:
				case ETimingStatNames::InstallTime_BG:
					return ETimingStatNames::InstallTime_Real;

				//Intentional fallthrough for PSOTime
				case ETimingStatNames::PSOTime_Real:
				case ETimingStatNames::PSOTime_FG:
				case ETimingStatNames::PSOTime_BG:
					return ETimingStatNames::PSOTime_Real;

				default:
					break;
			}

			ensureAlwaysMsgf(false, TEXT("Missing PersistentStats::ETimingStatNames GetAssociatedRealTimerName entry! Missing Entry as Int: %s"), *LexToString(InTimerType));
			return ETimingStatNames::NumStatNames;
		}

		ETimingStatNames GetAssociatedFGTimerName(ETimingStatNames InTimerType)
		{
			switch (InTimerType)
			{
				//Intentional fallthrough for TotalTime
				case ETimingStatNames::TotalTime_Real:
				case ETimingStatNames::TotalTime_FG:
				case ETimingStatNames::TotalTime_BG:
					return ETimingStatNames::TotalTime_FG;

				//Intentional fallthrough for ChunkDBDownloadTime
				case ETimingStatNames::ChunkDBDownloadTime_Real:
				case ETimingStatNames::ChunkDBDownloadTime_FG:
				case ETimingStatNames::ChunkDBDownloadTime_BG:
					return ETimingStatNames::ChunkDBDownloadTime_FG;

				//Intentional fallthrough for InstallTime
				case ETimingStatNames::InstallTime_Real:
				case ETimingStatNames::InstallTime_FG:
				case ETimingStatNames::InstallTime_BG:
					return ETimingStatNames::InstallTime_FG;

				//Intentional fallthrough for PSOTime
				case ETimingStatNames::PSOTime_Real:
				case ETimingStatNames::PSOTime_FG:
				case ETimingStatNames::PSOTime_BG:
					return ETimingStatNames::PSOTime_FG;

				default:
					break;
			}

			ensureAlwaysMsgf(false, TEXT("Missing PersistentStats::ETimingStatNames GetAssociatedFGTimerName entry! Missing Entry as Int: %s"), *LexToString(InTimerType));
			return ETimingStatNames::NumStatNames;
		}

		ETimingStatNames GetAssociatedBGTimerName(ETimingStatNames InTimerType)
		{
			switch (InTimerType)
			{
				//Intentional fallthrough for TotalTime
				case ETimingStatNames::TotalTime_Real:
				case ETimingStatNames::TotalTime_FG:
				case ETimingStatNames::TotalTime_BG:
					return ETimingStatNames::TotalTime_BG;

				//Intentional fallthrough for ChunkDBDownloadTime
				case ETimingStatNames::ChunkDBDownloadTime_Real:
				case ETimingStatNames::ChunkDBDownloadTime_FG:
				case ETimingStatNames::ChunkDBDownloadTime_BG:
					return ETimingStatNames::ChunkDBDownloadTime_BG;

				//Intentional fallthrough for InstallTime
				case ETimingStatNames::InstallTime_Real:
				case ETimingStatNames::InstallTime_FG:
				case ETimingStatNames::InstallTime_BG:
					return ETimingStatNames::InstallTime_BG;

				//Intentional fallthrough for PSOTime
				case ETimingStatNames::PSOTime_Real:
				case ETimingStatNames::PSOTime_FG:
				case ETimingStatNames::PSOTime_BG:
					return ETimingStatNames::PSOTime_BG;

			default:
				break;
			}

			ensureAlwaysMsgf(false, TEXT("Missing PersistentStats::ETimingStatNames GetAssociatedBGTimerName entry! Missing Entry as Int: %s"), *LexToString(InTimerType));
			return ETimingStatNames::NumStatNames;
		}

		bool FPersistentStatsBase::LoadStatsFromDisk()
		{
			if (!bHasLoadedFromDisk)
			{
				FString JSONStringOnDisk;
				if (FPaths::FileExists(GetFullPathForStatFile()))
				{
					FFileHelper::LoadFileToString(JSONStringOnDisk, *GetFullPathForStatFile());
				}

				if (!JSONStringOnDisk.IsEmpty())
				{
					bHasLoadedFromDisk = FromJson(JSONStringOnDisk);
					
					if (bHasLoadedFromDisk)
					{
						OnLoadingDataFromDisk();
					}
					
					return bHasLoadedFromDisk;
				}
			}
			
			return false;
		}

		bool FPersistentStatsBase::SaveStatsToDisk()
		{
			bIsDirty = false;
			return FFileHelper::SaveStringToFile(ToJson(), *GetFullPathForStatFile());
		}

		void FPersistentStatsBase::ResetStats(const FString& NewAnalyticsSessionID)
		{
			TimingStatsMap.Reset();
			CountStatMap.Reset();
			AnalyticsSessionID = NewAnalyticsSessionID;
			
			bIsDirty = true;
		}

		bool FPersistentStatsBase::HasTimingStat(ETimingStatNames StatToCheck) const
		{
			const FPersistentTimerData* FoundStat = TimingStatsMap.Find(LexToString(StatToCheck));
			return (nullptr != FoundStat);
		}

		bool FPersistentStatsBase::HasCountStat(ECountStatNames StatToCheck) const
		{
			const int* FoundStat = CountStatMap.Find(LexToString(StatToCheck));
			return (nullptr != FoundStat);
		}

		const FPersistentTimerData* FPersistentStatsBase::GetTimingStatData(ETimingStatNames StatToGet) const
		{ 
			return TimingStatsMap.Find(LexToString(StatToGet));
		}

		const int* FPersistentStatsBase::GetCountStatData(ECountStatNames StatToGet) const
		{
			return CountStatMap.Find(LexToString(StatToGet));
		}

		void FPersistentStatsBase::IncrementCountStat(PersistentStats::ECountStatNames StatToUpdate)
		{
			int& StatCount = CountStatMap.FindOrAdd(LexToString(StatToUpdate));
			++StatCount;

			bIsDirty = true;
		}

		bool FPersistentStatsBase::IsTimingStatStarted(PersistentStats::ETimingStatNames StatToUpdate) const
		{
			bool HasStarted = false;

			if (HasTimingStat(StatToUpdate))
			{
				const FPersistentTimerData* FoundStat = GetTimingStatData(StatToUpdate);
				if (ensureAlwaysMsgf((nullptr != FoundStat), TEXT("Missing FInstallBundlePersistentTimingData but returned true from HasTimingStat For Stat:%s!"), *LexToString(StatToUpdate)))
				{
					HasStarted = (FoundStat->LastUpdateTime != 0.);
				}
			}

			return HasStarted;
		}

		void FPersistentStatsBase::StartTimingStat(PersistentStats::ETimingStatNames StatToUpdate)
		{
			if (!IsTimingStatStarted(StatToUpdate))
			{
				FPersistentTimerData& FoundStat = TimingStatsMap.FindOrAdd(LexToString(StatToUpdate));
				FoundStat.LastUpdateTime = FPlatformTime::Seconds();
			}
			//if this stat was already updated then instead of losing the time since its last update by
			//starting it again lets just update it to keep that time
			else
			{
				UpdateTimingStat(StatToUpdate);
			}
			bIsDirty = true;
		}

		void FPersistentStatsBase::StopTimingStat(PersistentStats::ETimingStatNames StatToUpdate, bool UpdateTimerOnStop /* = true */)
		{
			//Only want to actually update the timer if we have started it (otherwise the update won't do anything and will ensure)
			if (UpdateTimerOnStop && IsTimingStatStarted(StatToUpdate))
			{
				UpdateTimingStat(StatToUpdate);
			}

			FPersistentTimerData& FoundStat = TimingStatsMap.FindOrAdd(LexToString(StatToUpdate));
			FoundStat.LastUpdateTime = 0.;

			bIsDirty = true;
		}

		void FPersistentStatsBase::UpdateTimingStat(PersistentStats::ETimingStatNames StatToUpdate)
		{
			if (ensureAlwaysMsgf(IsTimingStatStarted(StatToUpdate), TEXT("Calling UpdateTimingStat on a stat that hasn't been started! %s"), *LexToString(StatToUpdate)))
			{
				FPersistentTimerData& FoundStat = TimingStatsMap.FindOrAdd(LexToString(StatToUpdate));

				const double CurrentTime = FPlatformTime::Seconds();
				const double TimeSinceUpdate = CurrentTime - FoundStat.LastUpdateTime;

				if (ensureAlwaysMsgf((TimeSinceUpdate > 0.f), TEXT("Logic Error! Invalid saved LastUpdateTime for Stat %s!"), *LexToString(StatToUpdate)))
				{
					FoundStat.CurrentValue += TimeSinceUpdate;
				}
				
				FoundStat.LastUpdateTime = CurrentTime;
				bIsDirty = true;
			}
		}

		void FPersistentStatsBase::UpdateAllActiveTimers()
		{
			for (uint8 TimingStatNameIndex = 0; TimingStatNameIndex < (uint8)PersistentStats::ETimingStatNames::NumStatNames; ++TimingStatNameIndex)
			{
				PersistentStats::ETimingStatNames EnumForIndex = (PersistentStats::ETimingStatNames)TimingStatNameIndex;
				if (IsTimingStatStarted(EnumForIndex))
				{
					UpdateTimingStat(EnumForIndex);
				}
			}
		}

		void FPersistentStatsBase::StopAllActiveTimers()
		{
			for (uint8 TimingStatIndex = 0; TimingStatIndex < static_cast<uint8>(ETimingStatNames::NumStatNames); ++TimingStatIndex)
			{
				ETimingStatNames TimingStatAsEnum = static_cast<ETimingStatNames>(TimingStatIndex);
				if (IsTimingStatStarted(TimingStatAsEnum))
				{
					StopTimingStat(TimingStatAsEnum);
				}
			}
		}

		void FPersistentStatsBase::StatsBegin(const FString& ExpectedAnalyticsID, bool bForceResetData /* = false */)
		{
			bIsActive = true;
			
			LoadStatsFromDisk();

			//If our Analytics ID doesn't match our expected we need to reset the data as we have started a new persistent session
			if (bForceResetData || !AnalyticsSessionID.Equals(ExpectedAnalyticsID))
			{
				ResetStats(ExpectedAnalyticsID);
			}
			
			//Immediately save here so we don't risk reloading the same stale data
			//if we don't make it to an update
			SaveStatsToDisk();
		}

		void FPersistentStatsBase::StatsEnd(bool bStopAllActiveTimers /* = true */)
		{
			bIsActive = false;

			if (bStopAllActiveTimers)
			{
				StopAllActiveTimers();
			}

			//Immediately save here as we only look to update active dirty bundles, and since
			//this likely won't be changed anymore we might as well save it out now
			SaveStatsToDisk();
		}

		void FPersistentStatsBase::OnLoadingDataFromDisk()
		{
			HandleTimerStatsAfterDataLoad();
		}

		void FPersistentStatsBase::HandleTimerStatsAfterDataLoad()
		{
			//Go through all timing stats and handle each one accordingly
			//All Real timers should be updated after load without being stopped.
			//All FG Timers we should stop these timers without updating them so that they don't accrue time from backgrounding
			//All BG timers Should be stopped, but update their timers on stopping so they accrue time from being inactive

			/*
					Handle Real Timers
			*/

			if (IsTimingStatStarted(ETimingStatNames::TotalTime_Real))
			{
				UpdateTimingStat(ETimingStatNames::TotalTime_Real);
			}

			if (IsTimingStatStarted(ETimingStatNames::ChunkDBDownloadTime_Real))
			{
				UpdateTimingStat(ETimingStatNames::ChunkDBDownloadTime_Real);
			}

			if (IsTimingStatStarted(ETimingStatNames::InstallTime_Real))
			{
				UpdateTimingStat(ETimingStatNames::InstallTime_Real);
			}

			if (IsTimingStatStarted(ETimingStatNames::PSOTime_Real))
			{
				UpdateTimingStat(ETimingStatNames::PSOTime_Real);
			}

			/*
					Handle Foreground Timers
			*/

			if (IsTimingStatStarted(ETimingStatNames::TotalTime_FG))
			{
				StopTimingStat(ETimingStatNames::TotalTime_FG, false);
			}

			if (IsTimingStatStarted(ETimingStatNames::ChunkDBDownloadTime_FG))
			{
				StopTimingStat(ETimingStatNames::ChunkDBDownloadTime_FG, false);
			}

			if (IsTimingStatStarted(ETimingStatNames::InstallTime_FG))
			{
				StopTimingStat(ETimingStatNames::InstallTime_FG, false);
			}

			if (IsTimingStatStarted(ETimingStatNames::PSOTime_FG))
			{
				StopTimingStat(ETimingStatNames::PSOTime_FG, false);
			}

			/*
					Handle Background Timers
			*/
			
			if (IsTimingStatStarted(ETimingStatNames::TotalTime_BG))
			{
				StopTimingStat(ETimingStatNames::TotalTime_BG, true);
			}

			if (IsTimingStatStarted(ETimingStatNames::ChunkDBDownloadTime_BG))
			{
				StopTimingStat(ETimingStatNames::ChunkDBDownloadTime_BG, true);
			}

			if (IsTimingStatStarted(ETimingStatNames::InstallTime_BG))
			{
				StopTimingStat(ETimingStatNames::InstallTime_BG, true);
			}

			if (IsTimingStatStarted(ETimingStatNames::PSOTime_BG))
			{
				StopTimingStat(ETimingStatNames::PSOTime_BG, true);
			}
		}

		void FSessionPersistentStats::AddRequiredBundles(const TArray<FString>& RequiredBundlesToAdd)
		{
			for (const FString& BundleName : RequiredBundlesToAdd)
			{
				RequiredBundles.AddUnique(BundleName);
			}
			
			bIsDirty = true;
		}

		void FSessionPersistentStats::AddRequiredBundles(const TArray<FName>& RequiredBundlesToAdd)
		{
			for (FName BundleName : RequiredBundlesToAdd)
			{
				RequiredBundles.AddUnique(BundleName.ToString());
			}
			bIsDirty = true;
		}

		void FSessionPersistentStats::GetRequiredBundles(TArray<FString>& OutRequiredBundles) const
		{
			OutRequiredBundles.Empty();
			for (const FString& BundleName : RequiredBundles)
			{
				OutRequiredBundles.Add(BundleName);
			}
		}
	
		void FSessionPersistentStats::ResetRequiredBundles(const TArray<FString>& NewRequiredBundles /* = TArray<FString>() */)
		{
			RequiredBundles.Empty();
			AddRequiredBundles(NewRequiredBundles);
			bIsDirty = true;
		}

		const FString FBundlePersistentStats::GetFullPathForStatFile() const
		{
			return FPaths::Combine(FPlatformMisc::GamePersistentDownloadDir(), TEXT("PersistentStats"), TEXT("BundleStats"), (BundleName + TEXT(".json")));
		}

		const FString FSessionPersistentStats::GetFullPathForStatFile() const
		{
			return FPaths::Combine(FPlatformMisc::GamePersistentDownloadDir(), TEXT("PersistentStats"), TEXT("SessionStats"), (SessionName + TEXT(".json")));
		}

		FPersistentStatContainerBase::FPersistentStatContainerBase()
			: PerBundlePersistentStatMap()
			, SessionPersistentStatMap()
			, TickHandle()
			, OnApp_EnteringForegroundHandle()
			, OnApp_EnteringBackgroundHandle()
			, TimerAutoUpdateTimeRemaining(10.0f)
			, TimerDirtyStatUpdateTimeRemaining(10.0f)
			, bShouldAutoUpdateTimersInTick(true)
			, TimerAutoUpdateRate(10.0f)
			, bShouldSaveDirtyStatsOnTick(true)
			, DirtyStatSaveToDiskRate(5.f)
			, bShouldAutoHandleFGBGStats(true)
		{
			InitializeBase();
		}

		FPersistentStatContainerBase::~FPersistentStatContainerBase()
		{
			ShutdownBase();
		}

		void FPersistentStatContainerBase::InitializeBase()
		{
			//Load Settings from Config
			{
				GConfig->GetBool(TEXT("InstallBundleManager.PersistentStatSettings"), TEXT("bShouldAutoUpdateTimersInTick"), bShouldAutoUpdateTimersInTick, GEngineIni);
				GConfig->GetFloat(TEXT("InstallBundleManager.PersistentStatSettings"), TEXT("TimerAutoUpdateRate"), TimerAutoUpdateRate, GEngineIni);

				GConfig->GetBool(TEXT("InstallBundleManager.PersistentStatSettings"), TEXT("bShouldSaveDirtyStatsOnTick"), bShouldSaveDirtyStatsOnTick, GEngineIni);
				GConfig->GetFloat(TEXT("InstallBundleManager.PersistentStatSettings"), TEXT("DirtyStatSaveToDiskRate"), DirtyStatSaveToDiskRate, GEngineIni);
				
				GConfig->GetBool(TEXT("InstallBundleManager.PersistentStatSettings"), TEXT("bShouldAutoHandleFGBGStats"), bShouldAutoHandleFGBGStats, GEngineIni);

				//Reset timers so they follow the new loaded-in value
				ResetTimerUpdate();
				ResetDirtyStatUpdate();
			}

			//Setup Delegates (Needs to happen after config to have AutoUpdate settings loaded)
			{
				//Only setup a tick function if we would use it
				if ((bShouldAutoUpdateTimersInTick || bShouldSaveDirtyStatsOnTick) && !TickHandle.IsValid())
				{
					TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FPersistentStatContainerBase::Tick));
				}

				//Only setup Foreground/Background delegates if we should be using them to swap stats
				if (bShouldAutoHandleFGBGStats)
				{
					if (!OnApp_EnteringForegroundHandle.IsValid())
					{
						OnApp_EnteringForegroundHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FPersistentStatContainerBase::OnApp_EnteringForeground);
					}
					if (!OnApp_EnteringBackgroundHandle.IsValid())
					{
						OnApp_EnteringBackgroundHandle = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FPersistentStatContainerBase::OnApp_EnteringBackground);
					}
				}
			}
		}

		void FPersistentStatContainerBase::ShutdownBase()
		{
			if (TickHandle.IsValid())
			{
				FTSTicker::RemoveTicker(TickHandle);
				TickHandle.Reset();
			}

			if (OnApp_EnteringForegroundHandle.IsValid())
			{
				FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(OnApp_EnteringForegroundHandle);
				OnApp_EnteringForegroundHandle.Reset();
			}
			if (OnApp_EnteringBackgroundHandle.IsValid())
			{
				FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(OnApp_EnteringBackgroundHandle);
				OnApp_EnteringBackgroundHandle.Reset();
			}
		}

		bool FPersistentStatContainerBase::Tick(float dt)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FPersistentStatContainerBase_Tick);
			if (bShouldAutoUpdateTimersInTick)
			{
				//Only update all active timers every TimerStat_ResetTimerValue seconds
				TimerAutoUpdateTimeRemaining -= dt;
				if (TimerAutoUpdateTimeRemaining < 0.0f)
				{
					ResetTimerUpdate();
					UpdateAllBundlesActiveTimers();
					UpdateAllSessionActiveTimers();
				}
			}

			if (bShouldSaveDirtyStatsOnTick)
			{
				TimerDirtyStatUpdateTimeRemaining -= dt;
				if (TimerDirtyStatUpdateTimeRemaining < 0.0f)
				{
					ResetDirtyStatUpdate();
					SaveAllDirtyStatsToDisk();
				}
			}

			//go ahead and always Tick once we start
			return true;
		}

		void FPersistentStatContainerBase::OnTimerStartedForStat(FPersistentStatsBase& BundleStatForTimer, ETimingStatNames TimerStarted)
		{
			//If we are auto handling FG/BG stats and we have a real timer, start the _FG version with the _Real version
			if (bShouldAutoHandleFGBGStats && IsTimerReal(TimerStarted))
			{
				BundleStatForTimer.StartTimingStat(GetAssociatedFGTimerName(TimerStarted));

				//We should also check to make sure the _BG version is stopped if it is running
				const ETimingStatNames BGTimingStat = GetAssociatedBGTimerName(TimerStarted);
				if (BundleStatForTimer.IsTimingStatStarted(BGTimingStat))
				{
					BundleStatForTimer.StopTimingStat(BGTimingStat, true);
				}
			}
		}

		void FPersistentStatContainerBase::OnTimerStoppedForStat(FPersistentStatsBase& BundleStatForTimer, ETimingStatNames TimerStarted)
		{
			//If we are auto handling FG/BG stats and we have a real timer, stop the _FG and _BG version with the _Real version
			if (bShouldAutoHandleFGBGStats && IsTimerReal(TimerStarted))
			{
				BundleStatForTimer.StopTimingStat(GetAssociatedFGTimerName(TimerStarted));
				BundleStatForTimer.StopTimingStat(GetAssociatedBGTimerName(TimerStarted));
			}
		}

		void FPersistentStatContainerBase::ResetTimerUpdate()
		{
			TimerAutoUpdateTimeRemaining = TimerAutoUpdateRate;
		}

		void FPersistentStatContainerBase::ResetDirtyStatUpdate()
		{
			TimerDirtyStatUpdateTimeRemaining = DirtyStatSaveToDiskRate;
		}		

		void FPersistentStatContainerBase::SaveAllDirtyStatsToDisk()
		{
			TArray<FName> AllBundleStatNames;
			PerBundlePersistentStatMap.GetKeys(AllBundleStatNames);
			for (FName& BundleName : AllBundleStatNames)
			{
				FBundlePersistentStats* BundleStats = PerBundlePersistentStatMap.Find(BundleName);
				check(BundleStats);
				if (BundleStats->IsDirty())
				{
					BundleStats->SaveStatsToDisk();
				}
			}

			TArray<FString> AllSessionStatNames;
			SessionPersistentStatMap.GetKeys(AllSessionStatNames);
			for (const FString& SessionName : AllSessionStatNames)
			{
				FSessionPersistentStats* SessionStats = SessionPersistentStatMap.Find(SessionName);
				check(SessionStats);
				if (SessionStats->IsDirty())
				{
					SessionStats->SaveStatsToDisk();
				}
			}			
		}

		void FPersistentStatContainerBase::RemoveSessionStats(const FString& SessionName)
		{
			SessionPersistentStatMap.Remove(SessionName);
		}
	
		void FPersistentStatContainerBase::RemoveBundleStats(FName BundleName)
		{
			PerBundlePersistentStatMap.Remove(BundleName);
		}
	
		void FPersistentStatContainerBase::StartBundlePersistentStatTracking(FName BundleName, const FString& ExpectedAnalyticsID /* = FString() */, bool bForceResetStatData /* = false */)
		{
			//Use the base expected analytics ID if one was not passed in
			const FString ExpectedAnalyticsToUse = ExpectedAnalyticsID.IsEmpty() ? FPersistentStatsBase::GetBaseExpectedAnalyticsID() : ExpectedAnalyticsID;

			FBundlePersistentStats& FoundBundleStats = PerBundlePersistentStatMap.FindOrAdd(BundleName, FBundlePersistentStats(BundleName));
			FoundBundleStats.StatsBegin(ExpectedAnalyticsToUse, bForceResetStatData);
		}

		void FPersistentStatContainerBase::StartSessionPersistentStatTracking(const FString& SessionName, const TArray<FName>& RequiredBundles /* = TArray<FName>() */, const FString& ExpectedAnalyticsID /* = FString() */, bool bForceResetStatData /* = false */)
		{
			//Use the base expected analytics ID if one was not passed in
			const FString ExpectedAnalyticsToUse = ExpectedAnalyticsID.IsEmpty() ? FPersistentStatsBase::GetBaseExpectedAnalyticsID() : ExpectedAnalyticsID;

			FSessionPersistentStats& FoundSessionStats = SessionPersistentStatMap.FindOrAdd(SessionName, FSessionPersistentStats(SessionName));
			FoundSessionStats.StatsBegin(ExpectedAnalyticsToUse, bForceResetStatData);

			//Also append starting required bundles as we may have new ones from the ones already in data
			FoundSessionStats.AddRequiredBundles(RequiredBundles);
			
			//Go ahead and load data for all bundles in our RequiredBundles list while we are starting our Session
			LoadRequiredBundleDataFromDiskForSession(SessionName);
		}
		
		void FPersistentStatContainerBase::StopBundlePersistentStatTracking(FName BundleName, bool bStopAllActiveTimers /* = true */)
		{
			FBundlePersistentStats* FoundBundleStats = PerBundlePersistentStatMap.Find(BundleName);
			if (nullptr != FoundBundleStats)
			{
				FoundBundleStats->StatsEnd(bStopAllActiveTimers);
			}
		}

		void FPersistentStatContainerBase::StopSessionPersistentStatTracking(const FString& SessionName, bool bStopAllActiveTimers /* = true */)
		{
			FSessionPersistentStats* FoundSessionStats = SessionPersistentStatMap.Find(SessionName);
			if (nullptr != FoundSessionStats)
			{
				FoundSessionStats->StatsEnd(bStopAllActiveTimers);
			}
		}

		void FPersistentStatContainerBase::LoadRequiredBundleDataFromDiskForSession(const FString& SessionName)
		{
			FSessionPersistentStats* FoundSessionStats = SessionPersistentStatMap.Find(SessionName);
			if (nullptr != FoundSessionStats)
			{
				TArray<FString> RequiredBundles;
				FoundSessionStats->GetRequiredBundles(RequiredBundles);
				
				for (const FString& Bundle : RequiredBundles)
				{
					FBundlePersistentStats* FoundBundleStats = PerBundlePersistentStatMap.Find(*Bundle);
					if (nullptr == FoundBundleStats)
					{
						FBundlePersistentStats NewBundleStats = FBundlePersistentStats(*Bundle);
						NewBundleStats.LoadStatsFromDisk();
						PerBundlePersistentStatMap.Emplace(*Bundle, MoveTemp(NewBundleStats));
					}
					else
					{
						FoundBundleStats->LoadStatsFromDisk();
					}
				}
			}
		}
	
		void FPersistentStatContainerBase::StartBundlePersistentStatTimer(FName BundleName, ETimingStatNames TimerToStart)
		{
			FBundlePersistentStats& FoundBundleStats = PerBundlePersistentStatMap.FindOrAdd(BundleName, FBundlePersistentStats(BundleName));
			FoundBundleStats.StartTimingStat(TimerToStart);

			ensureAlwaysMsgf(FoundBundleStats.IsActive(), TEXT("Invalid attempt to start %s on bundle %s that hasn't yet had StartBundlePersistentStatTracking called on it! Should always start tracking before using persistent stats!"), *LexToString(TimerToStart), *(BundleName.ToString()));
			
			OnTimerStartedForStat(FoundBundleStats, TimerToStart);
		}

		void FPersistentStatContainerBase::StartSessionPersistentStatTimer(const FString& SessionName, ETimingStatNames TimerToStart)
		{
			FSessionPersistentStats& FoundSessionStats = SessionPersistentStatMap.FindOrAdd(SessionName, FSessionPersistentStats(SessionName));
			FoundSessionStats.StartTimingStat(TimerToStart);

			ensureAlwaysMsgf(FoundSessionStats.IsActive(), TEXT("Invalid attempt to start %s on session %s that hasn't yet had StartBundlePersistentStatTracking called on it! Should always start tracking before using persistent stats!"), *LexToString(TimerToStart), *SessionName);
			
			OnTimerStartedForStat(FoundSessionStats, TimerToStart);
		}

		void FPersistentStatContainerBase::StopBundlePersistentStatTimer(FName BundleName, ETimingStatNames TimerToStop)
		{
			FBundlePersistentStats& FoundBundleStats = PerBundlePersistentStatMap.FindOrAdd(BundleName, FBundlePersistentStats(BundleName));
			FoundBundleStats.StopTimingStat(TimerToStop);

			OnTimerStoppedForStat(FoundBundleStats, TimerToStop);
		}

		void FPersistentStatContainerBase::StopSessionPersistentStatTimer(const FString& SessionName, ETimingStatNames TimerToStop)
		{
			FSessionPersistentStats& FoundSessionStats = SessionPersistentStatMap.FindOrAdd(SessionName, FSessionPersistentStats(SessionName));
			FoundSessionStats.StopTimingStat(TimerToStop);

			OnTimerStoppedForStat(FoundSessionStats, TimerToStop);
		}

		void FPersistentStatContainerBase::UpdateBundlePersistentStatTimer(FName BundleName, ETimingStatNames TimerToUpdate)
		{
			FBundlePersistentStats& FoundBundleStats = PerBundlePersistentStatMap.FindOrAdd(BundleName, FBundlePersistentStats(BundleName));
			FoundBundleStats.UpdateTimingStat(TimerToUpdate);
		}

		void FPersistentStatContainerBase::UpdateSessionPersistentStatTimer(const FString& SessionName, ETimingStatNames TimerToUpdate)
		{
			FSessionPersistentStats& FoundSessionStats = SessionPersistentStatMap.FindOrAdd(SessionName, FSessionPersistentStats(SessionName));
			FoundSessionStats.UpdateTimingStat(TimerToUpdate);
		}

		void FPersistentStatContainerBase::IncrementBundlePersistentCounter(FName BundleName, ECountStatNames CounterToUpdate)
		{
			FBundlePersistentStats& FoundBundleStats = PerBundlePersistentStatMap.FindOrAdd(BundleName, FBundlePersistentStats(BundleName));
			FoundBundleStats.IncrementCountStat(CounterToUpdate);

			ensureAlwaysMsgf(FoundBundleStats.IsActive(), TEXT("Invalid attempt to increment %s on bundle %s that hasn't yet had StartBundlePersistentStatTracking called on it! Should always start tracking before using persistent stats!"), *LexToString(CounterToUpdate), *(BundleName.ToString()));
		}

		void FPersistentStatContainerBase::IncrementSessionPersistentCounter(const FString& SessionName, ECountStatNames CounterToUpdate)
		{
			FSessionPersistentStats& FoundSessionStats = SessionPersistentStatMap.FindOrAdd(SessionName, FSessionPersistentStats(SessionName));
			FoundSessionStats.IncrementCountStat(CounterToUpdate);

			ensureAlwaysMsgf(FoundSessionStats.IsActive(), TEXT("Invalid attempt to increment %s on session %s that hasn't yet had StartBundlePersistentStatTracking called on it! Should always start tracking before using persistent stats!"), *LexToString(CounterToUpdate), *SessionName);
		}

		void FPersistentStatContainerBase::OnApp_EnteringBackground()
		{
			SCOPED_ENTER_BACKGROUND_EVENT(STAT_InstallBundle_OnApp_EnteringBackground);
			OnBackground_HandleBundleStats();
			OnBackground_HandleSessionStats();
		}

		void FPersistentStatContainerBase::OnApp_EnteringForeground()
		{
			OnForeground_HandleBundleStats();
			OnForeground_HandleSessionStats();
		}

		void FPersistentStatContainerBase::OnBackground_HandleBundleStats()
		{
			for (TPair<FName, FBundlePersistentStats>& BundlePair : PerBundlePersistentStatMap)
			{
				//Only bother updating bundles listed as active
				if (BundlePair.Value.IsActive())
				{
					UpdateStatsForBackground(BundlePair.Value);
				}
			}
		}

		void FPersistentStatContainerBase::OnForeground_HandleBundleStats()
		{
			for (TPair<FName, FBundlePersistentStats>& BundlePair : PerBundlePersistentStatMap)
			{
				//Only bother updating bundles listed as active
				if (BundlePair.Value.IsActive())
				{
					UpdateStatsForForeground(BundlePair.Value);
				}
			}
		}

		void FPersistentStatContainerBase::OnBackground_HandleSessionStats()
		{
			for (TPair<FString, FSessionPersistentStats>& SessionPair : SessionPersistentStatMap)
			{
				//Only bother updating sessions listed as active
				if (SessionPair.Value.IsActive())
				{
					UpdateStatsForBackground(SessionPair.Value);
				}
			}
		}

		void FPersistentStatContainerBase::OnForeground_HandleSessionStats()
		{
			for (TPair<FString, FSessionPersistentStats>& SessionPair : SessionPersistentStatMap)
			{
				//Only bother updating sessions listed as active
				if (SessionPair.Value.IsActive())
				{
					UpdateStatsForForeground(SessionPair.Value);
				}
			}
		}

		void FPersistentStatContainerBase::UpdateStatsForBackground(FPersistentStatsBase& StatToUpdate)
		{
			StatToUpdate.IncrementCountStat(ECountStatNames::NumBackgrounded);

			//Always handle ActiveTotalTime as this isn't dependent on what stage of the process we are in
			if (StatToUpdate.IsTimingStatStarted(ETimingStatNames::TotalTime_FG))
			{
				StatToUpdate.StartTimingStat(ETimingStatNames::TotalTime_BG);
				StatToUpdate.StopTimingStat(ETimingStatNames::TotalTime_FG);
			}

			//Besides the ActiveTotalTime above, we should only be in 1 of the following states at a time, so only handle the appropriate swap
			if (StatToUpdate.IsTimingStatStarted(ETimingStatNames::ChunkDBDownloadTime_FG))
			{
				StatToUpdate.StartTimingStat(ETimingStatNames::ChunkDBDownloadTime_BG);
				StatToUpdate.StopTimingStat(ETimingStatNames::ChunkDBDownloadTime_FG);
			}
			else if (StatToUpdate.IsTimingStatStarted(ETimingStatNames::InstallTime_FG))
			{
				StatToUpdate.StartTimingStat(ETimingStatNames::InstallTime_BG);
				StatToUpdate.StopTimingStat(ETimingStatNames::InstallTime_FG);
			}
			else if (StatToUpdate.IsTimingStatStarted(ETimingStatNames::PSOTime_FG))
			{
				StatToUpdate.StartTimingStat(ETimingStatNames::PSOTime_BG);
				StatToUpdate.StopTimingStat(ETimingStatNames::PSOTime_FG);
			}
		}

		void FPersistentStatContainerBase::UpdateStatsForForeground(FPersistentStatsBase& StatToUpdate)
		{
			StatToUpdate.IncrementCountStat(ECountStatNames::NumResumedFromBackground);

			//Always handle ActiveTotalTime as this isn't dependent on what stage of the process we are in
			if (StatToUpdate.IsTimingStatStarted(ETimingStatNames::TotalTime_BG))
			{
				StatToUpdate.StopTimingStat(ETimingStatNames::TotalTime_BG);
				StatToUpdate.StartTimingStat(ETimingStatNames::TotalTime_FG);
			}

			//Besides the ActiveTotalTime above, we should only be in 1 of the following states at a time, so only handle the appropriate swap
			if (StatToUpdate.IsTimingStatStarted(ETimingStatNames::ChunkDBDownloadTime_BG))
			{
				StatToUpdate.StopTimingStat(ETimingStatNames::ChunkDBDownloadTime_BG);
				StatToUpdate.StartTimingStat(ETimingStatNames::ChunkDBDownloadTime_FG);
			}
			else if (StatToUpdate.IsTimingStatStarted(ETimingStatNames::InstallTime_BG))
			{
				StatToUpdate.StopTimingStat(ETimingStatNames::InstallTime_BG);
				StatToUpdate.StartTimingStat(ETimingStatNames::InstallTime_FG);
			}
			else if (StatToUpdate.IsTimingStatStarted(ETimingStatNames::PSOTime_BG))
			{
				StatToUpdate.StopTimingStat(ETimingStatNames::PSOTime_BG);
				StatToUpdate.StartTimingStat(ETimingStatNames::PSOTime_FG);
			}
		}

		void FPersistentStatContainerBase::UpdateAllBundlesActiveTimers()
		{
			TArray<FName> BundleNames;
			PerBundlePersistentStatMap.GetKeys(BundleNames);
			
			for (FName BundleName : BundleNames)
			{
				InstallBundleUtil::PersistentStats::FBundlePersistentStats& BundleStats = PerBundlePersistentStatMap.FindOrAdd(BundleName, FBundlePersistentStats(BundleName));
				BundleStats.UpdateAllActiveTimers();
			}
		}

		void FPersistentStatContainerBase::UpdateAllSessionActiveTimers()
		{
			TArray<FString> SessionNames;
			SessionPersistentStatMap.GetKeys(SessionNames);

			for (const FString& SessionName : SessionNames)
			{
				InstallBundleUtil::PersistentStats::FSessionPersistentStats& SessionStats = SessionPersistentStatMap.FindOrAdd(SessionName, FSessionPersistentStats(SessionName));
				SessionStats.UpdateAllActiveTimers();
			}
		}

		const FString FPersistentStatsBase::GetBaseExpectedAnalyticsID()
		{
			const FString BaseExpectedAnalyticsID = FPlatformMisc::GetDeviceId() + TEXT("_") + FApp::GetBuildVersion();
			return BaseExpectedAnalyticsID;
		}

		const FBundlePersistentStats* FPersistentStatContainerBase::GetBundleStat(FName BundleName) const
		{
			return PerBundlePersistentStatMap.Find(BundleName);
		}

		const FSessionPersistentStats* FPersistentStatContainerBase::GetSessionStat(const FString& SessionName) const
		{
			return SessionPersistentStatMap.Find(SessionName);
		}
	}
}
