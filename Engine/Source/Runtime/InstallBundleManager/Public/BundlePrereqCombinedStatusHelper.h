// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "InstallBundleManagerInterface.h"
#include "InstallBundleTypes.h"
//#include "InstallBundleUtils.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FName;
class IInstallBundleManager;
struct FInstallBundlePauseInfo;
struct FInstallBundleProgress;
struct FInstallBundleRequestResultInfo;

//Handles calculating the bundle status by combining progress from all of its
//Prerequisites. Allows you to display one progress percent that is weighted based on all
//bundles' values.
class FInstallBundleCombinedProgressTracker
{
public:

	//Collapses all the bundle manager states into one of a few states so that you can show simple text based on this enum
	enum class ECombinedBundleStatus : int32
	{
		Unknown, 
		Initializing, 
		Updating, 
		Finishing, 
		Finished,
		Count
	};

	//provide all our needed combined status information in 1 struct
	struct FCombinedProgress
	{
		float BackgroundDownloadProgressPercent = 0.0f;
		float InstallOnlyProgressPercent = 0.0f;
		float ProgressPercent = 0.0f; // Download and Install progress combined
		int32 InstalledBundlesCount = 0;
		int32 TotalBundlesCount = 0;
		TArray<FBuildInstallerStat> Stats; // Used for additional information about the installs
		ECombinedBundleStatus CombinedStatus = ECombinedBundleStatus::Unknown;
		EInstallBundlePauseFlags CombinedPauseFlags = EInstallBundlePauseFlags::None;
		bool bIsPaused = false;
		bool bDoesCurrentStateSupportPausing = false;
		bool bBundleRequiresUpdate = false;
	};
	
public:
	/** bAutoTick Whehter this tracker should automatically Tick */
	INSTALLBUNDLEMANAGER_API FInstallBundleCombinedProgressTracker(bool bAutoTick = true, TUniqueFunction<void(const FCombinedProgress&)> OnTick = nullptr);
	INSTALLBUNDLEMANAGER_API ~FInstallBundleCombinedProgressTracker();
	
	INSTALLBUNDLEMANAGER_API FInstallBundleCombinedProgressTracker(const FInstallBundleCombinedProgressTracker& Other);
	INSTALLBUNDLEMANAGER_API FInstallBundleCombinedProgressTracker(FInstallBundleCombinedProgressTracker&& Other);
	
	INSTALLBUNDLEMANAGER_API FInstallBundleCombinedProgressTracker& operator=(const FInstallBundleCombinedProgressTracker& Other);
	INSTALLBUNDLEMANAGER_API FInstallBundleCombinedProgressTracker& operator=(FInstallBundleCombinedProgressTracker&& Other);
	
	//Setup tracking for all bundles required in the supplied BundleContentState
	INSTALLBUNDLEMANAGER_API void SetBundlesToTrackFromContentState(const FInstallBundleCombinedContentState& BundleContentState, TArrayView<FName> BundlesToTrack);
	
	//Get current CombinedBundleStatus for everything setup to track
	INSTALLBUNDLEMANAGER_API const FCombinedProgress& GetCurrentCombinedProgress() const;
	
	//Useful for resolving tick order issue
	void ForceTick() { Tick(0); }

private:
	INSTALLBUNDLEMANAGER_API bool Tick(float dt);
	INSTALLBUNDLEMANAGER_API void UpdateBundleCache();
	INSTALLBUNDLEMANAGER_API void UpdateCombinedStatus();
	
	INSTALLBUNDLEMANAGER_API void SetupDelegates(bool bAutoTick);
	INSTALLBUNDLEMANAGER_API void CleanUpDelegates();
	
	//Called so we can track when a bundle is finished
	INSTALLBUNDLEMANAGER_API void OnBundleInstallComplete(FInstallBundleRequestResultInfo CompletedBundleInfo);
	INSTALLBUNDLEMANAGER_API void OnBundleInstallPauseChanged(FInstallBundlePauseInfo PauseInfo);
	
	INSTALLBUNDLEMANAGER_API void UpdateCombinedProgress();
	
private:
	//All bundles we need including pre-reqs
	TArray<FName> RequiredBundleNames;
	
	//Internal Cache of all bundle statuses to track progress
	TMap<FName, FInstallBundleProgress> BundleStatusCache;
	
	//Bundle weights that determine what % of the overall install each bundle represents
	TMap<FName, float> CachedBundleWeights;
	
	FCombinedProgress CurrentCombinedProgress;
	
	TWeakPtr<IInstallBundleManager> InstallBundleManager;
	FTSTicker::FDelegateHandle TickHandle;
	TUniqueFunction<void(const FCombinedProgress&)> OnTick;
};

INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(FInstallBundleCombinedProgressTracker::ECombinedBundleStatus Status);
