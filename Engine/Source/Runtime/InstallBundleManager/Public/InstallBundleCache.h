// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "HAL/Platform.h"
#include "InstallBundleTypes.h"
#include "Misc/DateTime.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FInstallBundleCache;

struct FInstallBundleCacheInitInfo
{
	FName CacheName;
	uint64 Size = 0;
};

struct FInstallBundleCacheBundleInfo
{
	FName BundleName;
	uint64 FullInstallSize = 0; // Total disk footprint when this bundle is fully installed
	uint64 InstallOverheadSize = 0; // Any extra space required to update the bundle
	uint64 CurrentInstallSize = 0; // Disk footprint of the bundle in it's current state
	FDateTime TimeStamp = FDateTime::MinValue(); // Last access time for the bundle.  Used for eviction order
	double AgeScalar = 1.0; // Allow some bundles to "age" slower than others
};

enum class EInstallBundleCacheReserveResult : int8
{
	Fail_CacheFull, // Cache is full and it's not possible to evict anything else from the cache
	Fail_NeedsEvict, // Cache is full but it' possible to evict released bundles to make room for this one
	Fail_PendingEvict, // This bundle can't be reserved because it's currently being evicted
	Success, // Bundle was reserved successfully
};

struct FInstallBundleCacheReserveResult
{
	TMap<FName, TArray<FInstallBundleSourceType>> BundlesToEvict;
	EInstallBundleCacheReserveResult Result = EInstallBundleCacheReserveResult::Success;
};

struct FInstallBundleCacheFlushResult
{
	TMap<FName, TArray<FInstallBundleSourceType>> BundlesToEvict;
};

class FInstallBundleCache : public TSharedFromThis<FInstallBundleCache>
{
public:
	INSTALLBUNDLEMANAGER_API virtual ~FInstallBundleCache();

	INSTALLBUNDLEMANAGER_API void Init(FInstallBundleCacheInitInfo InitInfo);

	FName GetName() const { return CacheName; }

	// Add a bundle to the cache.  
	INSTALLBUNDLEMANAGER_API void AddOrUpdateBundle(FInstallBundleSourceType Source, const FInstallBundleCacheBundleInfo& AddInfo);

	INSTALLBUNDLEMANAGER_API void RemoveBundle(FInstallBundleSourceType Source, FName BundleName);

	INSTALLBUNDLEMANAGER_API TOptional<FInstallBundleCacheBundleInfo> GetBundleInfo(FName BundleName) const;
	INSTALLBUNDLEMANAGER_API TOptional<FInstallBundleCacheBundleInfo> GetBundleInfo(FInstallBundleSourceType Source, FName BundleName) const;	

	// Return the total size of the cache
	INSTALLBUNDLEMANAGER_API uint64 GetSize() const;
	// Return the amount of space in use.  This could possbly exceed GetSize() if the cache size is changed or more 
	// bundles are added the to cache.
	INSTALLBUNDLEMANAGER_API uint64 GetUsedSize() const;
	// Return the amount of free space in the cache, clamped to [0, GetSize()]
	INSTALLBUNDLEMANAGER_API uint64 GetFreeSpace() const;

	// Called from bundle manager
	INSTALLBUNDLEMANAGER_API FInstallBundleCacheReserveResult Reserve(FName BundleName);

	// Called from bundle manager, returns all bundles that can be evicted
	INSTALLBUNDLEMANAGER_API FInstallBundleCacheFlushResult Flush(FInstallBundleSourceType* Source = nullptr);

	INSTALLBUNDLEMANAGER_API bool Contains(FName BundleName) const;
	INSTALLBUNDLEMANAGER_API bool Contains(FInstallBundleSourceType Source, FName BundleName) const;

	INSTALLBUNDLEMANAGER_API bool IsReserved(FName BundleName) const;

	// Called from bundle manager to make the files for this bundle eligible for eviction
	INSTALLBUNDLEMANAGER_API bool Release(FName BundleName);

	INSTALLBUNDLEMANAGER_API bool SetPendingEvict(FName BundleName);

	INSTALLBUNDLEMANAGER_API bool ClearPendingEvict(FName BundleName);

	// Hint to the cache that this bundle is requested, and we should prefer to evict non-requested bundles if possible
	INSTALLBUNDLEMANAGER_API void HintRequested(FName BundleName, bool bRequested);

	INSTALLBUNDLEMANAGER_API FInstallBundleCacheStats GetStats(EInstallBundleCacheStatsFlags Flags = EInstallBundleCacheStatsFlags::None, bool bVerbose = false) const;

private:
	INSTALLBUNDLEMANAGER_API uint64 GetFreeSpaceInternal(uint64 UsedSize) const;

	INSTALLBUNDLEMANAGER_API void CheckInvariants() const;
	
	INSTALLBUNDLEMANAGER_API void UpdateCacheInfoFromSourceInfo(FName BundleName);

private:
	struct FPerSourceBundleCacheInfo
	{
		uint64 FullInstallSize = 0;
		uint64 InstallOverheadSize = 0;
		uint64 CurrentInstallSize = 0;
		FDateTime TimeStamp = FDateTime::MinValue();
		double AgeScalar = 1.0;
	};

	enum class ECacheState : uint8
	{
		Released, //Transitions to Reserved or PendingEvict
		Reserved, //Transitions to Released
		PendingEvict, // Transitions to Released
	};

	struct FBundleCacheInfo 
	{
		uint64 FullInstallSize = 0;
		uint64 InstallOverheadSize = 0;
		uint64 CurrentInstallSize = 0;
		FDateTime TimeStamp = FDateTime::MinValue();
		double AgeScalar = 1.0;
		ECacheState State = ECacheState::Released;
		int32 HintReqeustedCount = 0; // Hint to the cache that this bundle is requested, and we should prefer to evict non-requested bundles if possible

		bool IsHintRequested() const { return HintReqeustedCount > 0; }

		uint64 GetSize() const
		{
			if (State == ECacheState::Released)
				return CurrentInstallSize;

			// Just consider any pending evictions to be 0 size.
			// Bundle Manager will still wait on them if necessary when reserving.
			if (State == ECacheState::PendingEvict)
				return 0;

			if (CurrentInstallSize > FullInstallSize)
				return CurrentInstallSize + InstallOverheadSize;

			return FullInstallSize + InstallOverheadSize;
		}
	};

	struct FCacheSortPredicate
	{
		bool operator()(const FBundleCacheInfo& A, const FBundleCacheInfo& B) const
		{
			if (A.IsHintRequested() == B.IsHintRequested())
			{
				FTimespan AgeA = (Now > A.TimeStamp) ? Now - A.TimeStamp : FTimespan(0);
				FTimespan AgeB = (Now > B.TimeStamp) ? Now - B.TimeStamp : FTimespan(0);

				return AgeA * A.AgeScalar > AgeB * B.AgeScalar;
			}

			return !A.IsHintRequested() && B.IsHintRequested();
		};

	private:
		FDateTime Now = FDateTime::UtcNow();
	};

private:

	TMap<FName, TMap<FInstallBundleSourceType, FPerSourceBundleCacheInfo>> PerSourceCacheInfo;

	// mutable to allow sorting in const contexts
	mutable TMap<FName, FBundleCacheInfo> CacheInfo;

	uint64 TotalSize = 0;

	FName CacheName;
};
