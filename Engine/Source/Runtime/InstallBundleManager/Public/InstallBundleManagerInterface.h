// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Union.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "InstallBundleTypes.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class IAnalyticsProviderET;
class IInstallBundleSource;
struct FInstallManagerBundleReport;

namespace InstallBundleUtil
{
	struct FContentRequestSharedContext;
	using FContentRequestSharedContextPtr = TSharedPtr<FContentRequestSharedContext>;
}

struct FInstallBundleProgress
{
	FName BundleName;

	EInstallBundleStatus Status = EInstallBundleStatus::Requested;

	EInstallBundlePauseFlags PauseFlags = EInstallBundlePauseFlags::None;

	float BackgroundDownload_Percent = 0;
	float InstallOnly_Percent = 0;
	float Install_Percent = 0; // Download and Install progress combined
	
	TArray<FBuildInstallerStat> Stats; // Used for additional information about the install

	float Finishing_Percent = 0;
};

struct FInstallBundleRequestResultInfo
{
	FName BundleName;
	EInstallBundleResult Result = EInstallBundleResult::OK;
	bool bIsStartup = false;
	bool bContainsChunks = false;
	bool bContainsIoStoreOnDemand = false;
	bool bContentWasInstalled = false;

	// Currently, these just forward BPT Error info
	FText OptionalErrorText;
	FString OptionalErrorCode;
};

struct FInstallBundleReleaseRequestResultInfo
{
	FName BundleName;
	EInstallBundleReleaseResult Result = EInstallBundleReleaseResult::OK; 
};

struct FInstallBundleRequestInfo
{
	EInstallBundleRequestInfoFlags InfoFlags = EInstallBundleRequestInfoFlags::None;
	TArray<FName> BundlesEnqueued;
	TArray<FInstallBundleRequestResultInfo> BundleResults;
};

struct FInstallBundleReleaseRequestInfo
{
	EInstallBundleRequestInfoFlags InfoFlags = EInstallBundleRequestInfoFlags::None;
	TArray<FName> BundlesEnqueued;
};

struct FInstallBundlePauseInfo
{
	FName BundleName;
	EInstallBundlePauseFlags PauseFlags = EInstallBundlePauseFlags::None;
};

enum class EInstallBundleManagerInitErrorHandlerResult
{
	NotHandled, // Defer to the next handler
	Retry, // Try to initialize again
	StopInitialization, // Stop trying to initialize
};

struct FInstallBundleChunkDownloadInfo
{
	FName BundleName;
	FString ChunkName;
	int32 ChunkFileSize;
	float ChunkDownloadDuration;
	FDateTime FetchStartTimeUTC;
	FDateTime FetchEndTimeUTC;
};

using FInstallBundleSourceOrCache = TUnion<FInstallBundleSourceType, FName>;

DECLARE_DELEGATE_RetVal_OneParam(EInstallBundleManagerInitErrorHandlerResult, FInstallBundleManagerInitErrorHandler, EInstallBundleManagerInitResult);
DECLARE_MULTICAST_DELEGATE_OneParam(FInstallBundleManagerInitCompleteMultiDelegate, EInstallBundleManagerInitResult);

DECLARE_MULTICAST_DELEGATE_OneParam(FInstallBundleChunkDownloadMetricsMultiDelegate, FInstallBundleChunkDownloadInfo);
DECLARE_MULTICAST_DELEGATE_OneParam(FInstallBundleCompleteMultiDelegate, FInstallBundleRequestResultInfo);
DECLARE_MULTICAST_DELEGATE_OneParam(FInstallBundlePausedMultiDelegate, FInstallBundlePauseInfo);
DECLARE_MULTICAST_DELEGATE_OneParam(FInstallBundleReleasedMultiDelegate, FInstallBundleReleaseRequestResultInfo);

DECLARE_MULTICAST_DELEGATE_OneParam(FInstallBundleManagerOnPatchCheckComplete, EInstallBundleManagerPatchCheckResult);

DECLARE_MULTICAST_DELEGATE_OneParam(FInstallBundleManagerReportingDelegate, const FInstallManagerBundleReport&);

DECLARE_DELEGATE_RetVal(bool, FInstallBundleManagerEnvironmentWantsPatchCheck);

DECLARE_DELEGATE_OneParam(FInstallBundleGetInstallStateDelegate, FInstallBundleCombinedInstallState);

DECLARE_DELEGATE(FInstallBundleManagerFlushCacheCompleteDelegate);
DECLARE_DELEGATE(FInstallBundleManagerAllDownloadsCompleted);

class IInstallBundleManager : public TSharedFromThis<IInstallBundleManager>
{
public:
	static INSTALLBUNDLEMANAGER_API FInstallBundleManagerInitCompleteMultiDelegate InitCompleteDelegate;

	static INSTALLBUNDLEMANAGER_API FInstallBundleChunkDownloadMetricsMultiDelegate InstallBundleChunkDownloadMetricsDelegate; // Called when a content request metrics is available
	static INSTALLBUNDLEMANAGER_API FInstallBundleCompleteMultiDelegate InstallBundleCompleteDelegate; // Called when a content request is complete
	static INSTALLBUNDLEMANAGER_API FInstallBundlePausedMultiDelegate PausedBundleDelegate;
	static INSTALLBUNDLEMANAGER_API FInstallBundleReleasedMultiDelegate ReleasedDelegate; // Called when content release request is complete
	static INSTALLBUNDLEMANAGER_API FInstallBundleManagerOnPatchCheckComplete PatchCheckCompleteDelegate;
	static INSTALLBUNDLEMANAGER_API FInstallBundleManagerReportingDelegate ReportingDelegate;
	static INSTALLBUNDLEMANAGER_API FInstallBundleManagerAllDownloadsCompleted AllDownloadsCompletedDelegate; // Called when all content downloads completed

	static INSTALLBUNDLEMANAGER_API TSharedPtr<IInstallBundleManager> GetPlatformInstallBundleManager();

	virtual ~IInstallBundleManager() = default;

	virtual void Initialize() {}

	virtual bool HasBundleSource(FInstallBundleSourceType SourceType) const = 0;

	virtual INSTALLBUNDLEMANAGER_API void SetContext(FName ContextName);
	virtual INSTALLBUNDLEMANAGER_API bool CopyReportTo(FInstallManagerBundleReport& Report);

	INSTALLBUNDLEMANAGER_API virtual const TSharedPtr<IInstallBundleSource> GetBundleSource(FInstallBundleSourceType SourceType) const;

	// Note: Some bundles sources may call this from a background task, so it should be threadsafe
	virtual int32 SelectBundleSubset(const FString& BundleName, TConstArrayView<FString> BundleSubsets) const { return INDEX_NONE; }

	virtual FDelegateHandle PushInitErrorCallback(FInstallBundleManagerInitErrorHandler Callback) = 0;
	virtual void PopInitErrorCallback() = 0;
	virtual void PopInitErrorCallback(FDelegateHandle Handle) = 0;
	virtual void PopInitErrorCallback(FDelegateUserObjectConst InUserObject) = 0;

	virtual EInstallBundleManagerInitState GetInitState() const = 0;

	INSTALLBUNDLEMANAGER_API TValueOrError<FInstallBundleRequestInfo, EInstallBundleResult> RequestUpdateContent(FName BundleName, EInstallBundleRequestFlags Flags, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging, InstallBundleUtil::FContentRequestSharedContextPtr RequestSharedContext = nullptr);
	virtual TValueOrError<FInstallBundleRequestInfo, EInstallBundleResult> RequestUpdateContent(TArrayView<const FName> BundleNames, EInstallBundleRequestFlags Flags, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging, InstallBundleUtil::FContentRequestSharedContextPtr RequestSharedContext = nullptr) = 0;

	INSTALLBUNDLEMANAGER_API FDelegateHandle GetContentState(FName BundleName, EInstallBundleGetContentStateFlags Flags, bool bAddDependencies, FInstallBundleGetContentStateDelegate Callback, FName RequestTag = NAME_None);
	virtual FDelegateHandle GetContentState(TArrayView<const FName> BundleNames, EInstallBundleGetContentStateFlags Flags, bool bAddDependencies, FInstallBundleGetContentStateDelegate Callback, FName RequestTag = NAME_None) = 0;
	virtual void CancelAllGetContentStateRequestsForTag(FName RequestTag) = 0;
	virtual void CancelAllGetContentStateRequests(FDelegateHandle Handle) = 0;

	// Less expensive version of GetContentState() that only returns install state
	// Synchronous versions return null if bundle manager is not yet initialized
	INSTALLBUNDLEMANAGER_API FDelegateHandle GetInstallState(FName BundleName, bool bAddDependencies, FInstallBundleGetInstallStateDelegate Callback, FName RequestTag = NAME_None);
	virtual FDelegateHandle GetInstallState(TArrayView<const FName> BundleNames, bool bAddDependencies, FInstallBundleGetInstallStateDelegate Callback, FName RequestTag = NAME_None) = 0;
	INSTALLBUNDLEMANAGER_API TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> GetInstallStateSynchronous(FName BundleName, bool bAddDependencies) const;
	virtual TValueOrError<FInstallBundleCombinedInstallState, EInstallBundleResult> GetInstallStateSynchronous(TArrayView<const FName> BundleNames, bool bAddDependencies) const = 0;
	virtual EInstallBundleResult GetAllMountedBundles(TArray<FName>& OutBundleNames) const = 0;
	virtual EInstallBundleResult GetAllBundlesWithState(EInstallBundleInstallState InstallState, TArray<FName>& OutBundleNames) const = 0;
	virtual void CancelAllGetInstallStateRequestsForTag(FName RequestTag) = 0;
	virtual void CancelAllGetInstallStateRequests(FDelegateHandle Handle) = 0;

	INSTALLBUNDLEMANAGER_API TValueOrError<FInstallBundleReleaseRequestInfo, EInstallBundleResult> RequestReleaseContent(FName ReleaseName, EInstallBundleReleaseRequestFlags Flags, TArrayView<const FName> KeepNames = TArrayView<const FName>(), ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging);
	virtual TValueOrError<FInstallBundleReleaseRequestInfo, EInstallBundleResult> RequestReleaseContent(TArrayView<const FName> ReleaseNames, EInstallBundleReleaseRequestFlags Flags, TArrayView<const FName> KeepNames = TArrayView<const FName>(), ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) = 0;

	INSTALLBUNDLEMANAGER_API EInstallBundleResult FlushCache(FInstallBundleManagerFlushCacheCompleteDelegate Callback, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging);
	virtual EInstallBundleResult FlushCache(FInstallBundleSourceOrCache SourceOrCache, FInstallBundleManagerFlushCacheCompleteDelegate Callback, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) = 0;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use overload with EInstallBundleCacheStatsFlags")
	INSTALLBUNDLEMANAGER_API virtual TArray<FInstallBundleCacheStats> GetCacheStats(EInstallBundleCacheDumpToLog DumpToLog, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging);
	UE_DEPRECATED(5.7, "Use overload with EInstallBundleCacheStatsFlags")
	INSTALLBUNDLEMANAGER_API virtual TOptional<FInstallBundleCacheStats> GetCacheStats(FInstallBundleSourceOrCache SourceOrCache, EInstallBundleCacheDumpToLog DumpToLog, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual TArray<FInstallBundleCacheStats> GetCacheStats(EInstallBundleCacheStatsFlags Flags = EInstallBundleCacheStatsFlags::None, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) = 0;
	virtual TOptional<FInstallBundleCacheStats> GetCacheStats(FInstallBundleSourceOrCache SourceOrCache, EInstallBundleCacheStatsFlags Flags = EInstallBundleCacheStatsFlags::None, ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging) = 0;

	INSTALLBUNDLEMANAGER_API void RequestRemoveContentOnNextInit(FName RemoveName, TArrayView<const FName> KeepNames = TArrayView<const FName>());
	virtual void RequestRemoveContentOnNextInit(TArrayView<const FName> RemoveNames, TArrayView<const FName> KeepNames = TArrayView<const FName>()) = 0;

	INSTALLBUNDLEMANAGER_API void CancelRequestRemoveContentOnNextInit(FName BundleName);
	virtual void CancelRequestRemoveContentOnNextInit(TArrayView<const FName> BundleNames) = 0;

	virtual TArray<FName> GetRequestedRemoveContentOnNextInit() const = 0;

	INSTALLBUNDLEMANAGER_API void CancelUpdateContent(FName BundleName);
	virtual void CancelUpdateContent(TArrayView<const FName> BundleNames) = 0;

	INSTALLBUNDLEMANAGER_API void PauseUpdateContent(FName BundleName);
	virtual void PauseUpdateContent(TArrayView<const FName> BundleNames) = 0;

	INSTALLBUNDLEMANAGER_API void ResumeUpdateContent(FName BundleName);
	virtual void ResumeUpdateContent(TArrayView<const FName> BundleNames) = 0;

	virtual void RequestPausedBundleCallback() = 0;

	virtual TOptional<FInstallBundleProgress> GetBundleProgress(FName BundleName) const = 0;

	virtual EInstallBundleRequestFlags GetModifyableContentRequestFlags() const = 0;
	INSTALLBUNDLEMANAGER_API void UpdateContentRequestFlags(FName BundleName, EInstallBundleRequestFlags AddFlags, EInstallBundleRequestFlags RemoveFlags);
	virtual void UpdateContentRequestFlags(TArrayView<const FName> BundleNames, EInstallBundleRequestFlags AddFlags, EInstallBundleRequestFlags RemoveFlags) = 0;
	virtual void SetCellularPreference(int32 Value) = 0;
	virtual void SetLimitedDataPreference(int32 Value) = 0;
	
	// Unblocks any pending downloads that are waiting for opting into using cellular data.
	// Semantic wrapper until we get a better API. 
	virtual void UnblockAnyDownloadsWaitingForCellularOptIn()
	{
		// Some platforms like Android clears cellular preference internally after some time,
		// we need to constantly set it per request to unblock any pending downloads.
		SetCellularPreference(1);
	}

	virtual void SetCacheSize(FName CacheName, uint64 CacheSize) = 0;

	INSTALLBUNDLEMANAGER_API virtual void StartPatchCheck();
	virtual void AddEnvironmentWantsPatchCheckBackCompatDelegate(FName Tag, FInstallBundleManagerEnvironmentWantsPatchCheck Delegate) {}
	virtual void RemoveEnvironmentWantsPatchCheckBackCompatDelegate(FName Tag) {}
	virtual bool SupportsEarlyStartupPatching() const = 0;

	virtual bool IsNullInterface() const = 0;

	virtual void SetErrorSimulationCommands(const FString& CommandLine) {}

	virtual TSharedPtr<IAnalyticsProviderET> GetAnalyticsProvider() const { return TSharedPtr<IAnalyticsProviderET>(); }

	virtual void StartSessionPersistentStatTracking(const FString& SessionName, const TArray<FName>& RequiredBundles = TArray<FName>(), const FString& ExpectedAnalyticsID = FString(), bool bForceResetStatData = false, const FInstallBundleCombinedContentState* State = nullptr) {}
	virtual void StopSessionPersistentStatTracking(const FString& SessionName) {}

#if !UE_BUILD_SHIPPING
	virtual void GetDebugText(TArray<FString>& Output) {}
#endif

	virtual bool HasEverUpdatedContent() const { return true; }
};

