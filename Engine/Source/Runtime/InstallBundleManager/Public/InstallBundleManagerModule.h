// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/Platform.h"
#include "InstallBundleManagerInterface.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class IInstallBundleManager;

/**
 * Currently empty implementation for InstallBundleModule until things are moved in here.
 */
class FInstallBundleManagerModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

/**
 * Base Module Interface for InstallBundleManager implementation modules
 */
class IInstallBundleManagerModule : public IModuleInterface
{
public:
	virtual void PreUnloadCallback() override
	{
		InstallBundleManager = nullptr;
		// make sure the bundle manager was cleaned up
		check(LastInstallBundleManager.IsValid() == false);
	}

	TSharedPtr<IInstallBundleManager> GetInstallBundleManager()
	{
		return InstallBundleManager;
	}

protected:
	TSharedPtr<IInstallBundleManager> InstallBundleManager;
	TWeakPtr<IInstallBundleManager> LastInstallBundleManager;
};

/**
 * Module Interface for InstallBundleManager implementation modules
 */
template<class InstallBundleManagerModuleImpl>
class TInstallBundleManagerModule : public IInstallBundleManagerModule
{
public:
	virtual void StartupModule() override
	{
		// Only instantiate the bundle manager if this is the version the game has been configured to use
		FString ModuleName;
#if WITH_EDITOR
		GConfig->GetString(TEXT("InstallBundleManager"), TEXT("EditorModuleName"), ModuleName, GEngineIni);
#else
		GConfig->GetString(TEXT("InstallBundleManager"), TEXT("ModuleName"), ModuleName, GEngineIni);
#endif // WITH_EDITOR

		if (FModuleManager::Get().GetModule(*ModuleName) == this)
		{
			check(LastInstallBundleManager == nullptr);
			InstallBundleManager = MakeShared<InstallBundleManagerModuleImpl>();
			LastInstallBundleManager = InstallBundleManager;

			InstallBundleManager->Initialize();
		}
	}
};
