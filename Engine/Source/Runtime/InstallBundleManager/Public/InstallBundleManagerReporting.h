// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstallBundleUtils.h"

class IInstallBundleSource;

struct FInstallBundleReport : public FJsonSerializable
{
	// State that represents an updatable bundle, where a choice can be made to update in the background,
	// or without.
	struct FStateUpdatable : public FJsonSerializable
	{
		// The total amount of bytes to download for the bundle
		uint64 DownloadSize;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("DownloadSize", reinterpret_cast<int64&>(DownloadSize));
		END_JSON_SERIALIZER
	};

	// State that represents a bundle that is updating
	struct FStateUpdating : public FJsonSerializable
	{
		// The total amount of bytes that needs to be downloaded for the bundle
		uint64 DownloadSize;

		// Indicates if this bundle is being updated with background downloads or not
		bool bUsesBackgroundDownloads = false;

		// Indicates, in a value 0 <= x <= 1 the progress of the downloads. If this value reaches 1, the bundle
		// can be assumed to be installed.
		float DownloadProgress = 0;

		// Indicates how many bytes were downloaded in the background
		uint64 BackgroundDownloadedBytes;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("DownloadSize", reinterpret_cast<int64&>(DownloadSize));
			JSON_SERIALIZE("bUsesBackgroundDownloads", bUsesBackgroundDownloads);
			JSON_SERIALIZE("DownloadProgress", DownloadProgress);
			JSON_SERIALIZE("BackgroundDownloadedBytes", reinterpret_cast<int64&>(BackgroundDownloadedBytes));
		END_JSON_SERIALIZER
	};

	struct FStateInstalling : public FJsonSerializable
	{
		// The total amount of bytes that have been downloaded for the bundle
		uint64 DownloadedSize;

		// The progress of the installation, if available, otherwise probably always 0; once the bundle switches to
		// updated state, the InstallProgress is essentially 1
		float InstallProgress = 0;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("DownloadedSize", reinterpret_cast<int64&>(DownloadedSize));
			JSON_SERIALIZE("InstallProgress", InstallProgress);
		END_JSON_SERIALIZER
	};

	// State that represents a fully updated bundle that no longer needs to be updated
	struct FStateUpdated : public FJsonSerializable
	{
		// The total amount of bytes that have been downloaded for the bundle
		uint64 DownloadedSize;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("DownloadedSize", reinterpret_cast<int64&>(DownloadedSize));
		END_JSON_SERIALIZER
	};

	using FState = TVariant<FStateUpdatable, FStateUpdating, FStateInstalling, FStateUpdated>;

	// The name of the bundle
	FName BundleName;

	// A string specifying what content version is being upgraded FROM. The format of this string is specific for the source.
	// NOTE: if there is no known source version, this value should not be set.
	TOptional<FString> SourceVersion;

	// The state of the bundle
	TOptional<FState> State;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_NAME("Name", BundleName);
		JSON_SERIALIZE_OPTIONAL("SourceVersion", SourceVersion);
		JSON_SERIALIZE_OPTIONAL_VARIANT_BEGIN("State", State);
			JSON_SERIALIZE_VARIANT_IFTYPE_SERIALIZABLE("StateUpdatable", FStateUpdatable);
			JSON_SERIALIZE_VARIANT_IFTYPE_SERIALIZABLE("StateUpdating", FStateUpdating);
			JSON_SERIALIZE_VARIANT_IFTYPE_SERIALIZABLE("StateInstalling", FStateInstalling);
			JSON_SERIALIZE_VARIANT_IFTYPE_SERIALIZABLE("StateUpdated", FStateUpdated);
		JSON_SERIALIZE_OPTIONAL_VARIANT_END();
	END_JSON_SERIALIZER

	// Returns the total amount of bytes that will or has been downloaded for this bundle.
	INSTALLBUNDLEMANAGER_API uint64 TotalDownloadSize() const;

	// Returns the amount of downloaded bytes for this bundle, should always be 0 <= x <= TotalDownloadSize()
	INSTALLBUNDLEMANAGER_API uint64 DownloadedBytes() const;

	// Returns a value 0 <= x <= 1 to indicate the installation progress.
	// If the source doesn't support gradual installation progress, this will pop from 0 to 1.
	INSTALLBUNDLEMANAGER_API float InstallationProgress() const;
	
	// Adds missing bytes to content size and download progress
	INSTALLBUNDLEMANAGER_API void AddDiscrepancy(uint64 Bytes);
	
	bool IsUnknown() const { return !State.IsSet(); }
	bool IsUpdatable() const { return State.IsSet() && State->IsType<FStateUpdatable>(); }
	bool IsUpdating() const { return State.IsSet() && State->IsType<FStateUpdating>(); }
	bool IsInstalling() const { return State.IsSet() && State->IsType<FStateInstalling>(); }
	bool IsUpdated() const { return State.IsSet() && State->IsType<FStateUpdated>(); }
};

struct FInstallBundleSourceReport : public FJsonSerializable
{
	// The bundle source's type string
	FName SourceType;

	// A string specifying what content version is being upgrade TO. The format of this string is specific for the source.
	// NOTE: if there is no known target version, this value should not be set.
	TOptional<FString> TargetVersion;

	// Array containing a single bundle report for every bundle this source is handling.
	TArray<FInstallBundleReport> Bundles;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_NAME("SourceType", SourceType);
		JSON_SERIALIZE_OPTIONAL("TargetVersion", TargetVersion);
		JSON_SERIALIZE_ARRAY_SERIALIZABLE("Bundles", Bundles, FInstallBundleReport);
	END_JSON_SERIALIZER

	// Returns pointer to bundle report that's valid as long as Bundles is not modified.
	// Returns nullptr if there is no report yet for the given type.
	INSTALLBUNDLEMANAGER_API FInstallBundleReport* TryFindBundleReport(FName BundleName);

	// Returns a reference to the bundle source report for the given type. It adds one if it doesn't exist yet.
	// The reference is valid as long as SourceReports is not modified.
	INSTALLBUNDLEMANAGER_API FInstallBundleReport& FindOrAddBundleReport(FName BundleName);

	// Returns the total amount of bytes that will or has been downloaded for this bundle source for all bundles
	INSTALLBUNDLEMANAGER_API uint64 TotalDownloadSize() const;

	// Returns the amount of downloaded bytes for this bundle source for all bundles, should always be 0 <= x <= TotalDownloadSize()
	INSTALLBUNDLEMANAGER_API uint64 DownloadedBytes() const;

	// Returns the total amount of bundles being updated, regardless of their state
	INSTALLBUNDLEMANAGER_API int TotalBundleCount() const;

	// Returns the amount of bundles that has being updated
	INSTALLBUNDLEMANAGER_API int UpdatedBundleCount() const;
};

struct FInstallManagerBundleReport : public FJsonSerializable
{
	enum class ECombinedStatus
	{
		Unknown,		// if there are no bundle sources, or no bundles, or if at least one bundle is set to unknown
		Updatable,		// if none of the above, if all bundles are set to updatable
		Updating,		// if none of the above, if at least one bundle is still updating
		Finishing,		// if none of the above, if at least one bundle is still installing
		Finished,		// if none of the above, all bundles have been updated
	};

	// the context name for this bundle report
	FName ContextName;

	// Semi-unique session string, changes every time any bundle source decides to start updating to a new version; generated using UUID
	FString SessionId;

	// Indicates, if set, how many times the application was backgrounded during the update process for this bundle.
	// If not set, the bundle source does not keep track of this.
	TOptional<uint32> BackgroundedCount;

	// Indicates how many times this report has been loaded from disk; in other words, indicates how many times the update process was restarted.
	uint32 LoadedCount = 0;

	// Array containing a single source report for every unique source bundle the manager knows.
	TArray<FInstallBundleSourceReport> SourceReports;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_NAME("ContextName", ContextName);
		JSON_SERIALIZE("SessionId", SessionId);
		JSON_SERIALIZE_OPTIONAL("BackgroundedCount", BackgroundedCount);
		JSON_SERIALIZE("LoadedCount", LoadedCount);
		JSON_SERIALIZE_ARRAY_SERIALIZABLE("SourceReports", SourceReports, FInstallBundleSourceReport);
	END_JSON_SERIALIZER

	// Returns pointer to bundle source report that's valid as long as SourceReports is not modified.
	// Returns nullptr if there is no report yet for the given type.
	INSTALLBUNDLEMANAGER_API FInstallBundleSourceReport* TryFindBundleSourceReport(const FInstallBundleSourceType& Type);

	// Returns a reference to the bundle source report for the given type. It adds one if it doesn't exist yet.
	// The reference is valid as long as SourceReports is not modified.
	INSTALLBUNDLEMANAGER_API FInstallBundleSourceReport& FindOrAddBundleSourceReport(const FInstallBundleSourceType& Type);

	// Returns the download progress.
	// This returns a value 0 <= x <= 1.
	INSTALLBUNDLEMANAGER_API float GetDownloadProgress() const;

	// Returns the overall installation progress, which is separate from the download progress
	// This returns a value 0 <= x <= 1.
	INSTALLBUNDLEMANAGER_API float GetInstallationProgress() const;

	// Returns the total amount of bytes that will or has been downloaded.
	INSTALLBUNDLEMANAGER_API uint64 TotalDownloadSize() const;

	// Returns the amount of downloaded bytes, should always be 0 <= x <= TotalDownloadSize()
	INSTALLBUNDLEMANAGER_API uint64 DownloadedBytes() const;

	// Returns the total amount of bundles being updated, regardless of their state
	INSTALLBUNDLEMANAGER_API int TotalBundleCount() const;

	// Returns the amount of bundles that has being updated
	INSTALLBUNDLEMANAGER_API int UpdatedBundleCount() const;

	// Returns a progress string
	INSTALLBUNDLEMANAGER_API FString GetProgressString() const;

	// Returns a longer progress string with more information
	INSTALLBUNDLEMANAGER_API FString GetLongProgressString() const;

	// Returns a combined status of all bundles
	INSTALLBUNDLEMANAGER_API ECombinedStatus GetCombinedStatus() const;

	// Returns the lowest current versions of any bundle. If any bundle has no such version (in other words, nothing is installed),
	// or if there are no bundles, this returns an empty optional.
	INSTALLBUNDLEMANAGER_API TOptional<FString> GetLowestCurrentBundleVersion() const;

	// Returns the highest current versions of any bundle. If not a single bundle has a version set, this returns an empty optional.
	INSTALLBUNDLEMANAGER_API TOptional<FString> GetHighestCurrentBundleVersion() const;
};

DECLARE_DELEGATE(FInstallManagerBundleReportCacheDoneLoadingDelegate);

class FInstallManagerBundleReportCache
{
public:
	INSTALLBUNDLEMANAGER_API FInstallManagerBundleReportCache(TArray<TUniquePtr<InstallBundleUtil::FInstallBundleTask>>& AsyncTasks);

	// Attempts to load the cached bundle report from disk; calls the delegate when it's done loading.
	INSTALLBUNDLEMANAGER_API void Load(FInstallManagerBundleReportCacheDoneLoadingDelegate OnLoadCompleteCallback);

	// Returns the current report as cached on disk (or last Set)
	const FInstallManagerBundleReport& GetReport() const { return Report; }

	// Sets the new report, both in memory, as well as flushing it to disk on a separate thread automatically.
	INSTALLBUNDLEMANAGER_API void SetReport(FInstallManagerBundleReport NewReport);

private:
	FInstallManagerBundleReport Report;
	TArray<TUniquePtr<InstallBundleUtil::FInstallBundleTask>>& AsyncTasks;
	FString ReportFilename;
};
