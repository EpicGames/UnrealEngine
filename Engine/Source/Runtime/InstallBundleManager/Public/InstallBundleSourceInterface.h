// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstallBundleTypes.h"
#include "InstallBundleUtils.h"

class IInstallBundleSource;
class IAnalyticsProviderET;
struct FInstallBundleReport;

DECLARE_DELEGATE_TwoParams(FInstallBundleSourceInitDelegate, TSharedRef<IInstallBundleSource> /*Source*/, FInstallBundleSourceAsyncInitInfo /*InitInfo*/);

DECLARE_DELEGATE_TwoParams(FInstallBundleSourceQueryBundleInfoDelegate, TSharedRef<IInstallBundleSource> /*Source*/, FInstallBundleSourceBundleInfoQueryResult /*Result*/);
DECLARE_DELEGATE_RetVal_TwoParams(EInstallBundleSourceUpdateBundleInfoResult, FInstallBundleSourceUpdateBundleInfoDelegate, TSharedRef<IInstallBundleSource> /*Source*/, FInstallBundleSourceUpdateBundleInfoResult /*Result*/);
DECLARE_DELEGATE_TwoParams(FInstallBundleLostRelevanceForSourceDelegate, TSharedRef<IInstallBundleSource> /*Source*/, TSet<FName> /*BundleNames*/);

DECLARE_DELEGATE_TwoParams(FInstallBundleCompleteDelegate, TSharedRef<IInstallBundleSource> /*Source*/, FInstallBundleSourceUpdateContentResultInfo /*Result*/);
DECLARE_DELEGATE_TwoParams(FInstallBundlePausedDelegate, TSharedRef<IInstallBundleSource> /*Source*/, FInstallBundleSourcePauseInfo /*PauseInfo*/);
DECLARE_DELEGATE_TwoParams(FInstallBundleRemovedDelegate, TSharedRef<IInstallBundleSource> /*Source*/, FInstallBundleSourceReleaseContentResultInfo /*Result*/);

DECLARE_DELEGATE_OneParam(FInstallBundleReportingDelegate, const FInstallBundleReport&);

DECLARE_DELEGATE_FourParams(FInstallBundleRequestStartedDelegate, const FName& /*BundleName*/, const FString& /*ManifestName*/, const FString& /*ManifestPath*/, const FString& /*ManifestVersion*/);
DECLARE_DELEGATE_TwoParams(FInstallBundleRequestCompletedDelegate, const FName& /*BundleName*/, const FBundleRequestCompleteInfo& /*AnalyticInfo*/);

DECLARE_DELEGATE_TwoParams(FInstallBundleSourceContentPatchResultDelegate, TSharedRef<IInstallBundleSource> /*Source*/, bool /*bContentPatchRequired*/);

class IInstallBundleSource : public TSharedFromThis<IInstallBundleSource>
{
public:
	virtual ~IInstallBundleSource() = default;

	// Returns a unique id for this source
	virtual FInstallBundleSourceType GetSourceType() const = 0;

	// Returns the how this source should be weighted when combined with other sources
	virtual float GetSourceWeight() const { return 1.0f; }

	// Returns a value in the [0.1, 1.0] range that allows bundles using this source to age out of the LRU cache more slowly
	virtual double GetSourceCacheAgeScalar() const { return 1.0; }

	// Called once by bundle manager after constructing the bundle source
	// Any non-fallback errors returned will cause bundle manager to fail to initialize
	virtual FInstallBundleSourceInitInfo Init(
		TSharedRef<InstallBundleUtil::FContentRequestStatsMap> InRequestStats,
		TSharedPtr<IAnalyticsProviderET> AnalyticsProvider,
		TSharedPtr<InstallBundleUtil::PersistentStats::FPersistentStatContainerBase> PersistentStatsContainer) = 0;
	// Bundle manager will not call AsyncInit again until the bundle source calls back that it is complete
	// It will be retried indefinitely until init is successful.  
	virtual void AsyncInit(FInstallBundleSourceInitDelegate Callback) = 0;

	// Currently only called after AsyncInit initialization.
	// Provides information about bundles this source knows about back to bundle manager.
	virtual void AsyncInit_QueryBundleInfo(FInstallBundleSourceQueryBundleInfoDelegate OnCompleteCallback) = 0;

	// Sets callbacks to Install Bundle Manager for dynamically discovered bundles
	// UpdateCallback - Call to add or update bundle info from this source.  Must be called when there are no requests for the bundle.
	// LostRelevanceCallback - Allows bundle manager to clean up bundles if they are no longer relevant to any source
	virtual void AsyncInit_SetUpdateBundleInfoCallback(
		FInstallBundleSourceUpdateBundleInfoDelegate UpdateCallback, FInstallBundleLostRelevanceForSourceDelegate LostRelevanceCallback) {}

	virtual void OnBundleInfoPruned(FName InBundleName) {}

	// Whether this source has been initialized or not
	virtual EInstallBundleManagerInitState GetInitState() const = 0;

	// Returns content version in a "<BuildVersion>-<Platform>" format
	virtual FString GetContentVersion() const = 0;

	// Finds all dependencies for InBundleName, including InBundleName
	// SkippedUnknownBundles contains any unknown bundles that may have been found
	virtual TSet<FName> GetBundleDependencies(FName InBundleName, TSet<FName>* SkippedUnknownBundles = nullptr) const = 0;

	// Gets the state of content on disk
	// BundleNames contains all dependencies and has been deduped
	virtual void GetContentState(TArrayView<const FName> BundleNames, EInstallBundleGetContentStateFlags Flags, FInstallBundleGetContentStateDelegate Callback) = 0;

	// Allows this bundle source to reject bundle requests early, rather than failing them.
	// This means that client code does not have to wait on these bundles.
	// This may be called anytime after Init, even during AsyncInit
	virtual EInstallBundleSourceBundleSkipReason GetBundleSkipReason(FName BundleName) const { return EInstallBundleSourceBundleSkipReason::None; }

	struct FRequestUpdateContentBundleContext
	{
		FName BundleName;
		EInstallBundleRequestFlags Flags = EInstallBundleRequestFlags::None;
		ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging;
		FInstallBundlePausedDelegate PausedCallback;
		FInstallBundleCompleteDelegate CompleteCallback;
		TSharedPtr<InstallBundleUtil::FContentRequestSharedContext> RequestSharedContext;
		void* StatsCookie = nullptr;
	};

	// Updates content on disk if necessary
	// BundleContexts contains all dependencies and has been deduped
	virtual void RequestUpdateContent(FRequestUpdateContentBundleContext BundleContext) = 0;

	struct FRequestReleaseContentBundleContext
	{
		FName BundleName;
		EInstallBundleReleaseRequestFlags Flags = EInstallBundleReleaseRequestFlags::None;
		ELogVerbosity::Type LogVerbosityOverride = ELogVerbosity::NoLogging;
		FInstallBundleRemovedDelegate CompleteCallback;
	};

	// Notify bundle source that a bundle has been released
	// If EInstallBundleReleaseRequestFlags::RemoveFilesIfPossible is set, the source should remove content from disk if present
	// The source should set bContentWasRemoved to indicate to bundle manager that the bundle content was uninstalled
	// BundleContexts contains all dependencies and has been deduped
	// Bundle manager will not schedule removes at the same time as updates for the same bundle
	virtual void RequestReleaseContent(FRequestReleaseContentBundleContext BundleContext)
	{
		FInstallBundleSourceReleaseContentResultInfo ResultInfo;
		ResultInfo.BundleName = BundleContext.BundleName;
		BundleContext.CompleteCallback.Execute(AsShared(), MoveTemp(ResultInfo));
	}

	// Returns true if content is scheduled to be removed the next time the source is initialized
	// BundleNames contains all dependencies and has been deduped
	virtual bool RequestRemoveContentOnNextInit(TArrayView<const FName> RemoveNames) { return false; }
	// Call to cancel the removal of any content scheduled for removal the next time the source is initialized
	// Returns true if all bundles were canceled
	virtual bool CancelRequestRemoveContentOnNextInit(TArrayView<const FName> BundleNames) { return false; }
	// Returns the bundle names of any content that is currently removal the next time the source is initialized
	virtual TArray<FName> GetRequestedRemoveContentOnNextInit() const { return {}; }

	// Cancel the install for the specified bundles
	virtual void CancelBundles(TArrayView<const FName> BundleNames) {}
	virtual void CancelBundles(TArrayView<const FName> BundleNames, TArray<FName>& OutAdditionalBunldlesToCancel) { CancelBundles(BundleNames); }

	// User Pause/Resume bundles.
	virtual void UserPauseBundles(TArrayView<const FName> BundleNames) {}
	virtual void UserResumeBundles(TArrayView<const FName> BundleNames) {}

	// UpdateContentRequestFlags - Allow some flags to be updated for in flight requests
	// Currently only CheckForCellularDataUsage/CheckForLimitedDataUsage are supported
	virtual EInstallBundleRequestFlags GetModifyableContentRequestFlags() const { return EInstallBundleRequestFlags::None; }
	virtual void UpdateContentRequestFlags(TArrayView<const FName> BundleNames, EInstallBundleRequestFlags AddFlags, EInstallBundleRequestFlags RemoveFlags) {}
	virtual void SetCellularPreference(int32 Value) {}
	virtual void SetLimitedDataPreference(int32 Value) {}

	// Derived classes should implement this if their content install will take a significant amount of time
	virtual TOptional<FInstallBundleSourceProgress> GetBundleProgress(FName BundleName) const { return TOptional<FInstallBundleSourceProgress>(); }
	
	// Override to return information about a bundle installer
	virtual TArray<FBuildInstallerStat>& GetBundleUpdateStats(FName BundleName) { return EmptyStats; }

	virtual void CheckForContentPatch(FInstallBundleSourceContentPatchResultDelegate Callback) { Callback.Execute(AsShared(), false); }

	// Called by bundle manager to pass through command line options to simulate errors
	virtual void SetErrorSimulationCommands(const FString& CommandLine) {}

#if !UE_BUILD_SHIPPING
	virtual void GetDebugText(TArray<FString>& Output) {}
#endif

	virtual bool SupportsReportingDelegate() const { return false; }
	virtual void SetReportingDelegate(FInstallBundleReportingDelegate Delegate) {}
	
	virtual bool SupportsBundleRequestEventDelegates() const { return false; }
	virtual void SetBundleRequestStartedDelegate(FInstallBundleRequestStartedDelegate Delegate) {}
	virtual void SetBundleRequestCompletedDelegate(FInstallBundleRequestCompletedDelegate Delegate) {}

	// Called on the game thread whenever a new analytics provider is to be used than the one passed to Init()
	virtual void SetAnalyticsProvider(TSharedPtr<IAnalyticsProviderET> AnalyticsProvider) = 0;

private:
	TArray<FBuildInstallerStat> EmptyStats;
};
