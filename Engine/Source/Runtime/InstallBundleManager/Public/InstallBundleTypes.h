// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/EnumRange.h"
#include "Misc/TVariant.h"
#include "UObject/NameTypes.h"

#if !defined(WITH_PLATFORM_INSTALL_BUNDLE_SOURCE)
	#define WITH_PLATFORM_INSTALL_BUNDLE_SOURCE 0
#endif

enum class UE_DEPRECATED(5.5, "Use FInstallBundleSourceType") EInstallBundleSourceType : int
{
	Bulk,
	Launcher,
	BuildPatchServices,
#if WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
	Platform,
#endif // WITH_PLATFORM_INSTALL_BUNDLE_SOURCE
	GameCustom,
	Streaming,
	Count,
};
PRAGMA_DISABLE_DEPRECATION_WARNINGS
ENUM_RANGE_BY_COUNT(EInstallBundleSourceType, EInstallBundleSourceType::Count);
UE_DEPRECATED(5.5, "Use FInstallBundleSourceType")
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(EInstallBundleSourceType Type);
UE_DEPRECATED(5.5, "Use FInstallBundleSourceType")
INSTALLBUNDLEMANAGER_API void LexFromString(EInstallBundleSourceType& OutType, const TCHAR* String);
PRAGMA_ENABLE_DEPRECATION_WARNINGS


class FInstallBundleSourceType
{
private:
	FStringView NameStr;

public:
	INSTALLBUNDLEMANAGER_API explicit FInstallBundleSourceType(FStringView InNameStr);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	INSTALLBUNDLEMANAGER_API FInstallBundleSourceType(EInstallBundleSourceType InLegacySourceType);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FStringView GetName() const { return NameStr; }
	FString GetNameStr() const { return FString(NameStr); }
	const TCHAR* GetNameCStr() const { return NameStr.GetData(); }

	bool IsValid() const { return !NameStr.IsEmpty(); }

	bool operator==(const FInstallBundleSourceType Other) const
	{
		// These should always point to constant strings
		return NameStr.GetData() == Other.NameStr.GetData();
	}

	friend inline uint32 GetTypeHash(FInstallBundleSourceType In)
	{
		// These should always point to constant strings
		return PointerHash(In.NameStr.GetData());
	}
};
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(FInstallBundleSourceType Type);

enum class EInstallBundleManagerInitState : int
{
	NotInitialized,
	Failed,
	Succeeded
};

enum class EInstallBundleManagerInitResult : int
{
	OK,
	BuildMetaDataNotFound,
	RemoteBuildMetaDataNotFound,
	BuildMetaDataDownloadError,
	BuildMetaDataParsingError,
	DistributionRootParseError,
	DistributionRootDownloadError,
	ManifestArchiveError,
	ManifestCreationError,
	ManifestDownloadError,
	BackgroundDownloadsIniDownloadError,
	NoInternetConnectionError,
	ConfigurationError,
	ClientPatchRequiredError,
	Count
};
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(EInstallBundleManagerInitResult Result);

enum class EInstallBundleInstallState : int
{
	NotInstalled,
	NeedsUpdate,
	UpToDate,
	Count,
};
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(EInstallBundleInstallState State);

struct FInstallBundleCombinedInstallState
{
	TMap<FName, EInstallBundleInstallState> IndividualBundleStates;
	TSet<FName> MountedBundles;
	TSet<FName> BundlesWithIoStoreOnDemand;

	INSTALLBUNDLEMANAGER_API bool GetAllBundlesHaveState(EInstallBundleInstallState State, TArrayView<const FName> ExcludedBundles = TArrayView<const FName>()) const;
	INSTALLBUNDLEMANAGER_API bool GetAnyBundleHasState(EInstallBundleInstallState State, TArrayView<const FName> ExcludedBundles = TArrayView<const FName>()) const;
};

struct FInstallBundleContentState
{
	EInstallBundleInstallState State = EInstallBundleInstallState::NotInstalled;
	float Weight = 0.0f;
	TMap<FInstallBundleSourceType, FString> Version;
};

struct FInstallBundleContentSize
{
	// Size remaining to download
	uint64 DownloadSize = 0;
	// Size needed to install the remaining download under GamePersistentDownloadDir
	uint64 SpaceRequiredForInstall = 0;
	// Size needed to install the remaining download under other directories than GamePersistentDownloadDir
	uint64 SpaceRequiredForInstallOtherDirs = 0;
	// size of bundle currently on disk under GamePersistentDownloadDir
	uint64 CurrentSizeOnDisk = 0;
	// size of bundle currently on disk under other directories than GamePersistentDownloadDir
	uint64 CurrentSizeOnDiskOtherDirs = 0;

	FInstallBundleContentSize operator+(const FInstallBundleContentSize& Other) const
	{
		FInstallBundleContentSize Result;
		Result.DownloadSize = DownloadSize + Other.DownloadSize;
		Result.SpaceRequiredForInstall = SpaceRequiredForInstall + Other.SpaceRequiredForInstall;
		Result.SpaceRequiredForInstallOtherDirs = SpaceRequiredForInstallOtherDirs + Other.SpaceRequiredForInstallOtherDirs;
		Result.CurrentSizeOnDisk = CurrentSizeOnDisk + Other.CurrentSizeOnDisk;
		Result.CurrentSizeOnDiskOtherDirs = CurrentSizeOnDiskOtherDirs + Other.CurrentSizeOnDiskOtherDirs;
		return Result;
	}
	FInstallBundleContentSize& operator+=(const FInstallBundleContentSize& Other)
	{
		*this = *this + Other;
		return *this;
	}
};

// TODO: Create a per-source version of this struct so that its clear what data bundle sources
// are responsible for populating.
struct FInstallBundleCombinedContentState
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FInstallBundleCombinedContentState() = default;
	FInstallBundleCombinedContentState(FInstallBundleCombinedContentState&&) = default;
	FInstallBundleCombinedContentState& operator=(FInstallBundleCombinedContentState&&) = default;
	FInstallBundleCombinedContentState(const FInstallBundleCombinedContentState&) = default;
	FInstallBundleCombinedContentState& operator=(const FInstallBundleCombinedContentState&) = default;
	~FInstallBundleCombinedContentState() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	TMap<FName, FInstallBundleContentState> IndividualBundleStates;
	// Bundle Sources don't need to populate this, bundle manager can determine it from init data
	TSet<FName> BundlesWithIoStoreOnDemand;
	TMap<FInstallBundleSourceType, FString> CurrentVersion;
	
#if !UE_BUILD_SHIPPING
	UE_DEPRECATED(5.6, "Please switch to using ContentSize")
	uint64 DownloadSize = 0;
	UE_DEPRECATED(5.6, "Please switch to using ContentSize")
	uint64 InstallSize = 0;
	UE_DEPRECATED(5.6, "Please switch to using ContentSize")
	uint64 InstallOverheadSize = 0;
	UE_DEPRECATED(5.6, "Please switch to using ContentSize")
	uint64 MaxDiskSpaceRequired = 0;
#endif
	
	uint64 FreeSpace = 0;
	FInstallBundleContentSize ContentSize;

	UE_DEPRECATED(5.7, "Please switch to using ContentSize")
	TOptional<FInstallBundleContentSize> BackgroundDownloadContentSize;

	INSTALLBUNDLEMANAGER_API bool GetAllBundlesHaveState(EInstallBundleInstallState State, TArrayView<const FName> ExcludedBundles = TArrayView<const FName>()) const;	
	INSTALLBUNDLEMANAGER_API bool GetAnyBundleHasState(EInstallBundleInstallState State, TArrayView<const FName> ExcludedBundles = TArrayView<const FName>()) const;
};

enum class EInstallBundleGetContentStateFlags : uint32
{
	None = 0,
	ForceNoPatching UE_DEPRECATED(5.7, "ForceNoPatching is no longer supported, use uninstall") = (1 << 0),
	UpdateBundleReports = (1 << 1), // will update the BundleReport for the involved bundles (see InstallBundleManagerReporting.h and IInstallBundleManager::ReportingDelegate)
};
ENUM_CLASS_FLAGS(EInstallBundleGetContentStateFlags);

DECLARE_DELEGATE_OneParam(FInstallBundleGetContentStateDelegate, FInstallBundleCombinedContentState);

enum class EInstallBundleRequestInfoFlags : int32
{
	None							= 0,
	EnqueuedBundles					= (1 << 0),
	SkippedAlreadyMountedBundles	= (1 << 1),
	SkippedAlreadyUpdatedBundles	= (1 << 2), // Only possible with EInstallBundleRequestFlags::SkipMount
	SkippedAlreadyReleasedBundles	= (1 << 3),
	SkippedAlreadyRemovedBundles	= (1 << 4), // Only possible with EInstallBundleReleaseRequestFlags::RemoveFilesIfPossible
	SkippedUnknownBundles			= (1 << 5),
	SkippedInvalidBundles			= (1 << 6), // Bundle can't be used with this build
	SkippedUnusableLanguageBundles	= (1 << 7), // Can't enqueue language bundles because of current system settings
	SkippedBundlesDueToBundleSource	= (1 << 8), // A bundle source rejected a bundle for some reason
};
ENUM_CLASS_FLAGS(EInstallBundleRequestInfoFlags);

enum class EInstallBundleResult : uint32
{
	OK,
	FailedPrereqRequiresLatestClient,
	FailedPrereqRequiresLatestContent,
	FailedCacheReserve,
	InstallError,
	InstallerOutOfDiskSpaceError,
	ManifestArchiveError,
	ConnectivityError,
	UserCancelledError,
	InitializationError,
	InitializationPending,
	MetadataError,
	Count,
};
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(EInstallBundleResult Result);

// TODO: Should probably be renamed to EInstallBundleRequestUpdateFlags 
enum class EInstallBundleRequestFlags : uint32
{
	None											= 0,
	CheckForCellularDataUsage						= (1 << 0),
	UseBackgroundDownloads UE_DEPRECATED(5.7, "UseBackgroundDownloads is no longer supported, done automatically") = (1 << 1),
	SendNotificationIfDownloadCompletesInBackground	= (1 << 2),
	ForceNoPatching UE_DEPRECATED(5.7, "ForceNoPatching is no longer supported, use uninstall") = (1 << 3),
	TrackPersistentBundleStats						= (1 << 4),
	SkipMount										= (1 << 5),
	SkipOnDemandTocMount							= (1 << 6),
	AsyncMount										= (1 << 7),
	UpdateBundleReports								= (1 << 8), // will update the BundleReport for the involved bundles (see InstallBundleManagerReporting.h and IInstallBundleManager::ReportingDelegate)
	ExplicitUpdateList								= (1 << 9),
	CheckForLimitedDataUsage						= (1 << 10),
	Defaults										= None,
};
ENUM_CLASS_FLAGS(EInstallBundleRequestFlags)

enum class EInstallBundleReleaseResult : uint32
{
	OK,
	ManifestArchiveError,
	UserCancelledError,
	MetadataError,
	Count,
};
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(EInstallBundleReleaseResult Result);

enum class EInstallBundleReleaseRequestFlags : uint32
{
	None = 0,
	RemoveFilesIfPossible = (1 << 0),  // Bundle sources must support removal, and bundle must not be part of the source's cache
	ExplicitRemoveList = (1 << 1),	   // Only attempt to remove explicitly supplied bundles instead of automatically removing dependencies
	SkipReleaseUnmountOnly = (1 << 2),   // Unmount but leave content referenced. The inverse of EInstallBundleRequestFlags::SkipMount
};
ENUM_CLASS_FLAGS(EInstallBundleReleaseRequestFlags)

enum class EInstallBundlePauseFlags : uint32
{
	None = 0,
	OnCellularNetwork    = (1 << 0),
	NoInternetConnection = (1 << 1),
	UserPaused           = (1 << 2),
	OnLimitedDataNetwork = (1 << 3)
};
ENUM_CLASS_FLAGS(EInstallBundlePauseFlags);

enum class EInstallBundleStatus : int
{
	Requested,
	Updating,
	Finishing,
	Ready,
	Count,
};
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(EInstallBundleStatus Status);

enum class EInstallBundleManagerPatchCheckResult : uint32
{
	/** No patch required */
	NoPatchRequired,
	/** Client Patch required to continue */
	ClientPatchRequired,
	/** Content Patch required to continue */
	ContentPatchRequired,
	/** Logged in user required for a patch check */
	NoLoggedInUser,
	/** Patch check failed */
	PatchCheckFailure,
	Count,
};
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(EInstallBundleManagerPatchCheckResult EnumVal);

/**
 * Enum used to describe download priority. Higher priorities will be downloaded first.
 * Note: Should always be kept in High -> Low priority order if adding more Priorities!
 */
enum class EInstallBundlePriority : uint8
{
	High,
	Normal,
	Low,
	Count
};
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(EInstallBundlePriority Priority);
INSTALLBUNDLEMANAGER_API bool LexTryParseString(EInstallBundlePriority& OutMode, const TCHAR* InBuffer);

struct FInstallBundleSourceInitInfo
{
	EInstallBundleManagerInitResult Result = EInstallBundleManagerInitResult::OK;
	bool bShouldUseFallbackSource = false;
};

struct FInstallBundleSourceAsyncInitInfo : public FInstallBundleSourceInitInfo
{
	// Reserved for future use
};

// Bundle Info communicated from bundle source to bundle manager at any time
struct FInstallBundleSourceUpdateBundleInfo
{
	FName BundleName;
	FString BundleNameString;
	EInstallBundlePriority Priority = EInstallBundlePriority::Low;
	uint64 FullInstallSize = 0; // Total disk footprint when this bundle is fully installed
	uint64 InstallOverheadSize = 0; // Any additional space required to complete installation
	FDateTime LastAccessTime = FDateTime::MinValue(); // If cached, used to decide eviction order
	EInstallBundleInstallState BundleContentState = EInstallBundleInstallState::NotInstalled; // Whether this bundle is up to date
	bool bIsCached = false; // Whether this bundle should be cached if this source has a bundle cache
	bool bContainsIoStoreOnDemandToc = false; // Whether this bundle contains an an IoStoreOnDemand Toc
};

struct FInstallBundleSourceUpdateBundleInfoResult
{
	TMap<FName, FInstallBundleSourceUpdateBundleInfo> SourceBundleInfoMap;
};

// Persisted Bundle Info communicated from bundle source to bundle manager on startup
struct FInstallBundleSourcePersistentBundleInfo : FInstallBundleSourceUpdateBundleInfo
{
	uint64 CurrentInstallSize = 0; // Disk footprint of the bundle in it's current state
	bool bIsStartup = false; // Only one startup bundle allowed.  All sources must agree on this.
	bool bDoPatchCheck = false; // This bundle should do a patch check and fail if it doesn't pas	
};

struct FInstallBundleSourceBundleInfoQueryResult
{
	TMap<FName, FInstallBundleSourcePersistentBundleInfo> SourceBundleInfoMap;
};

enum class EInstallBundleSourceUpdateBundleInfoResult : uint8
{
	OK,
	NotInitailized,
	AlreadyMounted,
	AlreadyRequested,
	IllegalCacheStatus,
	Count,
};
INSTALLBUNDLEMANAGER_API const TCHAR* LexToString(EInstallBundleSourceUpdateBundleInfoResult Result);

namespace UE::IoStore
{
	struct FOnDemandMountArgs;
}

struct FInstallBundleSourceUpdateContentResultInfo
{
	INSTALLBUNDLEMANAGER_API FInstallBundleSourceUpdateContentResultInfo();
	FInstallBundleSourceUpdateContentResultInfo(const FInstallBundleSourceUpdateContentResultInfo&) = delete;
	INSTALLBUNDLEMANAGER_API FInstallBundleSourceUpdateContentResultInfo(FInstallBundleSourceUpdateContentResultInfo&&);
	INSTALLBUNDLEMANAGER_API ~FInstallBundleSourceUpdateContentResultInfo();

	FName BundleName;
	EInstallBundleResult Result = EInstallBundleResult::OK;

	// Forward any errors from the underlying implementation for a specific source
	// Currently, these just forward BPT Error info
	FText OptionalErrorText;
	FString OptionalErrorCode;

	TArray<FString> ContentPaths;
	TArray<FString> AdditionalRootDirs;
	UE_DEPRECATED(5.6, "All shader libs are now packaged and avialable via UFS")
	TSet<FString> NonUFSShaderLibPaths;
	TArray<TUniquePtr<UE::IoStore::FOnDemandMountArgs>> OnDemandMountArgs;
	FPakMountOptions MountOptions;
	int32 PakOrder = INDEX_NONE;
	FString ProjectName;

	uint64 CurrentInstallSize = 0;
	FDateTime LastAccessTime = FDateTime::MinValue(); // If cached, used to decide eviction order

	bool bContentWasInstalled = false; // If true, the source did work to update the content
	
	bool DidBundleSourceDoWork() const { return (ContentPaths.Num() != 0);} 
};

struct FInstallBundleSourceReleaseContentResultInfo
{
	FName BundleName;
	EInstallBundleReleaseResult Result = EInstallBundleReleaseResult::OK;

	FDateTime LastAccessTime = FDateTime::MinValue(); // If cached, used to decide eviction order

	// Indicates content was actually removed and bundle manager should consider
	// this bundle as no longer installed.
	bool bContentWasRemoved = false;
};

// Useful to store any kind of information about the build installer
struct FBuildInstallerStat
{
	FName BundleName; // To known which Bundle this stat was for even after all the stats are aggregated
	FName StatName;
	TVariant<bool, int32, int64, float, double, FString> StatValue;
};

struct FInstallBundleSourceProgress
{
	FName BundleName;

	float BackgroundDownload_Percent = 0;
	float InstallOnly_Percent = -1; // -1 means the value is not valid and Install_Percent should be used instead
	float Install_Percent = 0; // Download and Install progress combined
	
	TArray<FBuildInstallerStat> Stats; // Used for additional information about the install
};

struct FInstallBundleSourcePauseInfo
{
	FName BundleName;
	EInstallBundlePauseFlags PauseFlags = EInstallBundlePauseFlags::None;
	// True if the bundle actually transitioned to/from paused,
	// which is different than the flags changing
	bool bDidPauseChange = false;
};

enum class EInstallBundleSourceBundleSkipReason : uint32
{
	None = 0,
	LanguageNotCurrent = (1 << 0), // The platform language must be changed to make it valid to request this bundle
	NotValid = (1 << 1), // Bundle can't be used with this build
};
ENUM_CLASS_FLAGS(EInstallBundleSourceBundleSkipReason);

struct FInstallBundleCacheBundleStats
{
	FName BundleName;
	uint64 FullInstallSize = 0;
	uint64 InstallOverheadSize = 0;
	uint64 CurrentInstallSize = 0;
	FDateTime TimeStamp = FDateTime::MinValue();
	double AgeScalar = 1.0;
	bool bReserved = false;
};

struct FInstallBundleCacheStats
{
	FName CacheName;
	uint64 MaxSize = 0;
	uint64 UsedSize = 0;
	uint64 ReservedSize = 0;
	uint64 FreeSize = 0;
	// EInstallBundleCacheStatsFlags::DumpToResults must be used to populate BundleStats
	TArray<FInstallBundleCacheBundleStats> BundleStats;
};

enum class EInstallBundleCacheStatsFlags : uint8
{
	None = 0,
	DumpToLog = (1 << 0),
	CSVFormat = (1 << 1),
	DumpToResults = (1 << 2)
};
ENUM_CLASS_FLAGS(EInstallBundleCacheStatsFlags);

enum class UE_DEPRECATED(5.7, "Use EInstallBundleCacheStatsFlags") EInstallBundleCacheDumpToLog : int8
{
	None = 0,
	Default,
	CSV
};

// TODO: These details shouldn't be in engine code. If you find yourself adding to this struct, please consider refactoring this 
// to pass back a list of FAnalyticsEventAttribute's instead.
struct FBundleDecompressionInfo
{
	uint64 TotalCompressedBytes{};				// total amount of bytes in compressed files (incl headers)
	uint64 TotalDecompressedBytes{};			// total amount of bytes in decompressed files (incl headers)
	uint64 TotalActuallyReadBytes{};			// total amount of compressed bytes actually read this session (excl headers)
	uint64 TotalActuallyWrittenBytes{};			// total amount of decompressed bytes actually written this session (excl headers)
	double TotalDecompressionReadTime{};		// seconds spent reading compressed bytes this session
	double TotalDecompressionDecompressTime{};	// seconds spent decompressing this session
	double TotalDecompressionWriteTime{};		// seconds spent writing decompressed bytes this session
};

// TODO: These details shouldn't be in engine code. If you find yourself adding to this struct, please consider refactoring this 
// to pass back a list of FAnalyticsEventAttribute's instead.
struct FBundleRequestCompleteInfo
{
	FString PreviousManifest;
	FString CurrentManifest;
	FString OldManifestVersion;
	FString InstallManifestVersion;
	FString OldVersionTimeStamp;
	uint64 TotalDownloadedBytes{};
	uint64 EstimatedFullDownloadBytes{};
	FString Result;
	FString BPTErrorCode;
	FString DownloadType;
	FBundleDecompressionInfo BundleDecompressionInfo;
	double AsyncStartupDelay{};
	double AsyncExitDelay{};
	bool AsyncPlacedInBackground{};
};
