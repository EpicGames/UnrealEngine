// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "InstallBundleTypes.h"
#include "Misc/EmbeddedCommunication.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/IsEnum.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Internationalization/Regex.h"

class FQueuedThreadPool;

namespace InstallBundleUtil
{
	// Returns the app version in the same format as BPS versions
	INSTALLBUNDLEMANAGER_API FString GetAppVersion();

	INSTALLBUNDLEMANAGER_API bool HasInternetConnection(ENetworkConnectionType ConnectionType);

	INSTALLBUNDLEMANAGER_API bool SplitHostUrl(const FStringView Url, FStringView& OutHost, FStringView& OutRemainder);

	INSTALLBUNDLEMANAGER_API const TCHAR* GetInstallBundlePauseReason(EInstallBundlePauseFlags Flags);

	INSTALLBUNDLEMANAGER_API const FString& GetInstallBundleSectionPrefix();

	INSTALLBUNDLEMANAGER_API const FString& GetInstallBundleSubsetSectionPrefix();

	INSTALLBUNDLEMANAGER_API bool GetConfiguredBundleSources(TArray<FString>& OutSources, TMap<FString, FString>& OutFallbackSources);

	// returns true if the given BundleName exists in the InstallBundle.ini config
	INSTALLBUNDLEMANAGER_API bool HasInstallBundleInConfig(const FString& BundleName);

	struct FConfigMountOptions
	{
		bool bWithSoftReferences = false; 
	};

	INSTALLBUNDLEMANAGER_API bool GetMountOptionsFromConfig(const FStringView BundleName, FConfigMountOptions& OutMountOptions);

	// Returns true
	INSTALLBUNDLEMANAGER_API bool AllInstallBundlePredicate(const FConfigFile& InstallBundleConfig, const FString& Section);

	// Returns true if this bundle should be included in a platform package
	INSTALLBUNDLEMANAGER_API bool IsPlatformInstallBundlePredicate(const FConfigFile& InstallBundleConfig, const FString& Section);

	// Returns an ordered regex list used to map files to bundles
	// SectionPredicate can be used to exclude non-relavent bundles from the list
	INSTALLBUNDLEMANAGER_API TArray<TPair<FString, TArray<FRegexPattern>>> LoadBundleRegexFromConfig(
		const FConfigFile& InstallBundleConfig,
		TFunctionRef<bool(const FConfigFile& InstallBundleConfig, const FString& Section)> SectionPredicate = AllInstallBundlePredicate);

	// Returns an ordered regex list used to map files to bundle subsets
	INSTALLBUNDLEMANAGER_API TArray<TPair<FString, TArray<FRegexPattern>>> LoadBundleSubsetRegexFromConfig(
		const FConfigFile& InstallBundleConfig);

	// Finds a matching install bundle for a given path
	INSTALLBUNDLEMANAGER_API bool MatchBundleRegex(
		const TArray<TPair<FString, TArray<FRegexPattern>>>& BundleRegexList,
		const FString& Path,
		FString& OutBundleName);

	// Return true for a matching install bundle subset for a given path
	INSTALLBUNDLEMANAGER_API bool MatchBundleSubsetRegex(
		const FString& BundleSubset,
		const TArray<TPair<FString, TArray<FRegexPattern>>>& BundleSubsetRegexList,
		const FString& Path);

	constexpr float MinimumBundleWeight = 0.05f;

	// It would really be nice to have these in core
	template<class EnumType>
	constexpr auto& CastAsUnderlying(EnumType &Type)
	{
		static_assert(TIsEnum<EnumType>::Value, "");
		using UnderType = std::underlying_type_t<EnumType>;
		return *reinterpret_cast<UnderType*>(&Type);
	}

	template<class EnumType>
	constexpr const auto& CastAsUnderlying(const EnumType &Type)
	{
		static_assert(TIsEnum<EnumType>::Value, "");
		using UnderType = std::underlying_type_t<EnumType>;
		return *reinterpret_cast<const UnderType*>(&Type);
	}

	template<class EnumType>
	constexpr auto CastToUnderlying(EnumType Type)
	{
		static_assert(TIsEnum<EnumType>::Value, "");
		using UnderType = std::underlying_type_t<EnumType>;
		return static_cast<UnderType>(Type);
	}

	template<typename EnumType, typename StringArrType, EnumType EnumCount = EnumType::Count>
	const TCHAR* TLexToString(EnumType E, const StringArrType& Strings)
	{
		constexpr auto Count = InstallBundleUtil::CastToUnderlying(EnumCount);
		static_assert(Count == UE_ARRAY_COUNT(Strings), "");

		auto Idx = InstallBundleUtil::CastToUnderlying(E);
		if (Idx >= 0 && Idx < Count)
		{
			return Strings[Idx];
		}
		else
		{
			return TEXT("");
		}
	}

	// Keep the engine awake via RAII when running as an embedded app
	class FInstallBundleManagerKeepAwake : public FEmbeddedKeepAwake
	{
		static INSTALLBUNDLEMANAGER_API FName Tag;
		static INSTALLBUNDLEMANAGER_API FName TagWithRendering;
	public:
		FInstallBundleManagerKeepAwake(bool bNeedsRendering = false)
			: FEmbeddedKeepAwake(bNeedsRendering ? TagWithRendering : Tag, bNeedsRendering) {}
	};

	class FInstallBundleManagerScreenSaverControl
	{
		static INSTALLBUNDLEMANAGER_API bool bDidDisableScreensaver;
		static INSTALLBUNDLEMANAGER_API int DisableCount;

		static INSTALLBUNDLEMANAGER_API void IncDisable();
		static INSTALLBUNDLEMANAGER_API void DecDisable();

	public:
		FInstallBundleManagerScreenSaverControl()
		{
			IncDisable();
		}

		~FInstallBundleManagerScreenSaverControl()
		{
			DecDisable();
		}

		FInstallBundleManagerScreenSaverControl(const FInstallBundleManagerScreenSaverControl& Other)
		{
			IncDisable();
		}
		FInstallBundleManagerScreenSaverControl(FInstallBundleManagerScreenSaverControl&& Other) = default;

		FInstallBundleManagerScreenSaverControl& operator=(const FInstallBundleManagerScreenSaverControl& Other) = default;
		FInstallBundleManagerScreenSaverControl& operator=(FInstallBundleManagerScreenSaverControl&& Other)
		{
			DecDisable();
			return *this;
		}
	};

	// Helper to safely suppress installer analytics that will restore previous state when going out of scope
	struct FInstallBundleSuppressAnalytics : FNoncopyable
	{
		INSTALLBUNDLEMANAGER_API FInstallBundleSuppressAnalytics();
		INSTALLBUNDLEMANAGER_API ~FInstallBundleSuppressAnalytics();

		INSTALLBUNDLEMANAGER_API void Enable();
		INSTALLBUNDLEMANAGER_API void Disable();

		static INSTALLBUNDLEMANAGER_API bool IsEnabled();

	private:
		bool bIsEnabled;
	};

	class FInstallBundleWork : public FNonAbandonableTask
	{
	public:
		FInstallBundleWork() = default;

		FInstallBundleWork(TUniqueFunction<void()> InWork, TUniqueFunction<void()> InOnComplete)
			: WorkFunc(MoveTemp(InWork))
			, OnCompleteFunc(MoveTemp(InOnComplete))
		{}

		void DoWork()
		{
			if (WorkFunc)
			{
				WorkFunc();
			}
		}

		void CallOnComplete()
		{
			if (OnCompleteFunc)
			{
				OnCompleteFunc();
			}
		}

		inline TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FInstallBundleWork, STATGROUP_ThreadPoolAsyncTasks);
		}

	private:
		TUniqueFunction<void()> WorkFunc;
		TUniqueFunction<void()> OnCompleteFunc;
	};

	using FInstallBundleTask = FAsyncTask<FInstallBundleWork>;
	using FAutoDeleteInstallBundleTask = FAutoDeleteAsyncTask<FInstallBundleWork>;

	INSTALLBUNDLEMANAGER_API void StartInstallBundleAsyncIOTask(TUniqueFunction<void()> WorkFunc);

	INSTALLBUNDLEMANAGER_API void StartInstallBundleAsyncIOTask(FQueuedThreadPool* ThreadPool, TUniqueFunction<void()> WorkFunc);

	INSTALLBUNDLEMANAGER_API void StartInstallBundleAsyncIOTask(TArray<TUniquePtr<FInstallBundleTask>>& Tasks, TUniqueFunction<void()> WorkFunc, TUniqueFunction<void()> OnComplete);

	INSTALLBUNDLEMANAGER_API void StartInstallBundleAsyncIOTask(FQueuedThreadPool* ThreadPool, TArray<TUniquePtr<FInstallBundleTask>>& Tasks, TUniqueFunction<void()> WorkFunc, TUniqueFunction<void()> OnComplete);

	INSTALLBUNDLEMANAGER_API void FinishInstallBundleAsyncIOTasks(TArray<TUniquePtr<FInstallBundleTask>>& Tasks);

	INSTALLBUNDLEMANAGER_API void CleanupInstallBundleAsyncIOTasks(TArray<TUniquePtr<FInstallBundleTask>>& Tasks);

	struct FContentRequestStateStats
	{
		double StartTime = 0.0;
		double EndTime = 0.0;
		uint64 DataSize = 0;
		bool bOpen = true;

		double GetElapsedTime() const
		{
			return (EndTime > StartTime) ? (EndTime - StartTime) : 0.0;
		}
	};

	struct FContentRequestStats
	{
		double StartTime = 0.0;
		double EndTime = 0.0;
		bool bOpen = true;
		TMap<FString, FContentRequestStateStats> StateStats;

		double GetElapsedTime() const
		{
			return (EndTime > StartTime) ? (EndTime - StartTime) : 0.0;
		}
	};

	class FContentRequestStatsKey
	{
	private:
		FName BundleName;
		void* Cookie = nullptr;

	public:
		FContentRequestStatsKey(FName InBundleName, void* InCookie)
			: BundleName(InBundleName), Cookie(InCookie)
		{}

		FName GetBundleName() const { return BundleName; }
		void* GetCookie() const { return Cookie; }

		[[nodiscard]] bool operator==(const FContentRequestStatsKey& Other) const
		{
			return BundleName == Other.BundleName && Cookie == Other.Cookie;
		}

		[[nodiscard]] friend uint32 GetTypeHash(const FContentRequestStatsKey& Key)
		{
			return HashCombineFast(GetTypeHash(Key.GetBundleName()), GetTypeHash(Key.GetCookie()));
		}
	};

	class FContentRequestStatsMap
	{
	private:
		TMap<FContentRequestStatsKey, FContentRequestStats> StatsMap;

	public:
		INSTALLBUNDLEMANAGER_API void StatsBegin(const FContentRequestStatsKey& Key);
		INSTALLBUNDLEMANAGER_API void StatsEnd(const FContentRequestStatsKey& Key);
		INSTALLBUNDLEMANAGER_API void StatsBegin(const FContentRequestStatsKey& Key, const TCHAR* State);
		INSTALLBUNDLEMANAGER_API void StatsEnd(const FContentRequestStatsKey& Key, const TCHAR* State, uint64 DataSize = 0);
		INSTALLBUNDLEMANAGER_API void StatsReset(const FContentRequestStatsKey& Key);

		const FContentRequestStats* FindStats(const FContentRequestStatsKey& Key) const
		{
			return StatsMap.Find(Key);
		}

		const FContentRequestStats& FindStatsChecked(const FContentRequestStatsKey& Key) const
		{
			return StatsMap.FindChecked(Key);
		}

		UE_DEPRECATED(5.7, "Use FContentRequestStatsKey version")
		void StatsBegin(FName BundleName) { StatsBegin(FContentRequestStatsKey(BundleName, nullptr)); }
		UE_DEPRECATED(5.7, "Use FContentRequestStatsKey version")
		void StatsEnd(FName BundleName) { StatsEnd(FContentRequestStatsKey(BundleName, nullptr)); }
		UE_DEPRECATED(5.7, "Use FContentRequestStatsKey version")
		void StatsBegin(FName BundleName, const TCHAR* State) { StatsBegin(FContentRequestStatsKey(BundleName, nullptr)); }
		UE_DEPRECATED(5.7, "Use FContentRequestStatsKey version")
		void StatsEnd(FName BundleName, const TCHAR* State, uint64 DataSize = 0) { StatsEnd(FContentRequestStatsKey(BundleName, nullptr)); }
		UE_DEPRECATED(5.7, "Use FContentRequestStatsKey version")
		void StatsReset(FName BundleName) { StatsReset(FContentRequestStatsKey(BundleName, nullptr)); }

		UE_DEPRECATED(5.7, "Use FindStats and FindStatsChecked")
		const TMap<FContentRequestStatsKey, FContentRequestStats>& GetMap() { return StatsMap; }
	};

	namespace PersistentStats
	{
		//Stats used to store timing information in FPersistentTimerData format
		//_Real stats account for total time since a stat was started regardless of the app's state (shutdown, in foreground, etc.)
		//_FG stats account for time while the app is active
		//_BG stats are an estimate of time while the app is inactive (backgrounded, closed, etc.)
		//		*_BG stats are an estimate as we can not account for unexpected crashes, etc.
		enum class ETimingStatNames : uint8
		{
			  TotalTime_Real
			, TotalTime_FG
			, TotalTime_BG
			, ChunkDBDownloadTime_Real
			, ChunkDBDownloadTime_FG
			, ChunkDBDownloadTime_BG
			, InstallTime_Real
			, InstallTime_FG
			, InstallTime_BG
			, PSOTime_Real
			, PSOTime_FG
			, PSOTime_BG
			, NumStatNames //should always be final entry
		};

		//Stats used to store count information in an int format
		enum class ECountStatNames : uint8
		{
			  NumResumedFromBackground
			, NumResumedFromLaunch
			, NumBackgrounded
			, NumStatNames //should always be final entry
		};

		INSTALLBUNDLEMANAGER_API const FString& LexToString(ETimingStatNames InType);
		INSTALLBUNDLEMANAGER_API const FString& LexToString(ECountStatNames InType);
		
		//Returns if the timer is in the _Real set
		INSTALLBUNDLEMANAGER_API bool IsTimerReal(ETimingStatNames InTimer);
		//Returns if the timer is in the _FG set
		INSTALLBUNDLEMANAGER_API bool IsTimerFG(ETimingStatNames InTimer);
		//Returns if the timer is in the _BG set
		INSTALLBUNDLEMANAGER_API bool IsTimerBG(ETimingStatNames InTimer);

		//Gets the _Real timer when supplied with the corresponding _FG or _BG timer name
		INSTALLBUNDLEMANAGER_API ETimingStatNames GetAssociatedRealTimerName(ETimingStatNames InTimerType);
		//Gets the _FG timer when supplied with the corresponding _Real or _BG timer name
		INSTALLBUNDLEMANAGER_API ETimingStatNames GetAssociatedFGTimerName(ETimingStatNames InTimerType);
		//Gets the _BG timer when supplied with the corresponding _Real or _FG timer name
		INSTALLBUNDLEMANAGER_API ETimingStatNames GetAssociatedBGTimerName(ETimingStatNames InTimerType);

		class FPersistentTimerData : public FJsonSerializable
		{
		public:
			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE("LastUpdateTime", LastUpdateTime);
				JSON_SERIALIZE("TotalTime", CurrentValue);
			END_JSON_SERIALIZER

		public:
			FPersistentTimerData()
				: LastUpdateTime(0.)
				, CurrentValue(0.)
			{}

			double LastUpdateTime;
			double CurrentValue;
		};

		//Both ContentRequests and Individual Bundles use this to track data respective to themselves.
		//Keeps us from having to write serialization / loading data for both cases that would be the same
		class FPersistentStatsBase : public FJsonSerializable
		{
		public:
			//We should not ever really be serializing this base class. Kept it here for reference.
			INSTALLBUNDLEMANAGER_API BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE("AnalyticsSessionID", AnalyticsSessionID);
				JSON_SERIALIZE_MAP("CountStats", CountStatMap);
				JSON_SERIALIZE_MAP_SERIALIZABLE("TimingStats", TimingStatsMap, FPersistentTimerData);
			END_JSON_SERIALIZER
		
		public:
			virtual ~FPersistentStatsBase() {}

			//Tries to load persistent stats from disk and determine if we need to resume our previous persistent session or start a new one
			INSTALLBUNDLEMANAGER_API void StatsBegin(const FString& ExpectedAnalyticsID, bool bForceResetData = false);

			//Tries to set this stat pool as "inactive". By default also stop all active timers. 
			// NOTE: If you don't stop all active timers future calls to UpdateTimingStat will still increment this value and save it off!
			INSTALLBUNDLEMANAGER_API void StatsEnd(bool bStopAllActiveTimers = true);

			INSTALLBUNDLEMANAGER_API bool IsTimingStatStarted(ETimingStatNames StatToUpdate) const;
			INSTALLBUNDLEMANAGER_API void StartTimingStat(ETimingStatNames StatToUpdate);
			INSTALLBUNDLEMANAGER_API void StopTimingStat(ETimingStatNames StatToUpdate, bool UpdateTimerOnStop = true);
			INSTALLBUNDLEMANAGER_API void UpdateTimingStat(ETimingStatNames StatToUpdate);
			
			INSTALLBUNDLEMANAGER_API void UpdateAllActiveTimers();
			INSTALLBUNDLEMANAGER_API void StopAllActiveTimers();
			
			bool IsActive() const { return bIsActive; }
			bool IsDirty() const { return bIsDirty; }

			INSTALLBUNDLEMANAGER_API void IncrementCountStat(PersistentStats::ECountStatNames StatToUpdate);

			INSTALLBUNDLEMANAGER_API bool HasTimingStat(ETimingStatNames StatToCheck) const;
			INSTALLBUNDLEMANAGER_API bool HasCountStat(ECountStatNames StatToCheck) const;

			INSTALLBUNDLEMANAGER_API const FPersistentTimerData* GetTimingStatData(ETimingStatNames StatToGet) const;
			INSTALLBUNDLEMANAGER_API const int* GetCountStatData(ECountStatNames StatToGet) const;

			//Saves the persistent stat data to disk in the location returned by GetFullPathForStatFile()
			//Returns True if that save succeeds and false otherwise.
			INSTALLBUNDLEMANAGER_API bool SaveStatsToDisk();
			
			//Function that allows you to load stats from disk. Returns true if stats were either loaded from disk now, or previously were loaded from disk.
			//NOTE: This does not begin stat collection, so you should still call StatsBegin on this before handling any stats!
			INSTALLBUNDLEMANAGER_API bool LoadStatsFromDisk();

		protected:
			INSTALLBUNDLEMANAGER_API void ResetStats(const FString& NewAnalyticsSessionID);
			
			//Called after we load data but before we process any of it.
			//Useful for fixing up the data loaded from disk before it changes or is acted upon by anything else
			//NOTE: A call to LoadStatsFromDisk will not necessarily call this! It will only be called when data is actually pulled from
			//the disk, and its possible to skip pulling data from the disk!
			INSTALLBUNDLEMANAGER_API void OnLoadingDataFromDisk();

			//Helper to try and reconcile offline and active timers after we load data from disk
			INSTALLBUNDLEMANAGER_API void HandleTimerStatsAfterDataLoad();

			//Protected so we can only construct the derived versions of this Base class
			FPersistentStatsBase()
				: TimingStatsMap()
				, CountStatMap()
				, AnalyticsSessionID()
				, bIsActive(false)
				, bIsDirty(false)
				, bHasLoadedFromDisk(false)
			{
			}
			
		protected:
			//Serialized Stat Data
			TMap<FString, FPersistentTimerData> TimingStatsMap;
			TMap<FString, int> CountStatMap;
			FString AnalyticsSessionID;

			//We don't serialize these as they are tracking behavior and not stats
			bool bIsActive;
			bool bIsDirty;
			
			bool bHasLoadedFromDisk;
			
		//Static Methods
		public:
			//Helper function that generates a basic ExpectedAnalyticsID from device and UE information
			static INSTALLBUNDLEMANAGER_API const FString GetBaseExpectedAnalyticsID();
		
		//Pure Virtual Methods
		public:
			//Returns Full Path to look for our persistent stat cache
			virtual const FString GetFullPathForStatFile() const = 0;
		};

		//Each bundles persistent stats
		class FBundlePersistentStats : public FPersistentStatsBase
		{
		public:
			INSTALLBUNDLEMANAGER_API BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE("BundleName", BundleName);
				JSON_SERIALIZE("AnalyticsSessionID", AnalyticsSessionID);
				JSON_SERIALIZE_MAP("CountStats", CountStatMap);
				JSON_SERIALIZE_MAP_SERIALIZABLE("TimingStats", TimingStatsMap, FPersistentTimerData);
			END_JSON_SERIALIZER


		private:
			//Remove default constructor as we require a BundleName in the constructor
			FBundlePersistentStats() = delete;

			//Used in our full path and should signify the FString version of the FName of this bundle we are tracking
			FString BundleName;

		public:
			FBundlePersistentStats(FName BundleNameIn)
				: BundleName(BundleNameIn.ToString())
			{
			}

			virtual ~FBundlePersistentStats() = default;

			//Implementation for FInstallBundleStatsContainerBase
		public:
			INSTALLBUNDLEMANAGER_API virtual const FString GetFullPathForStatFile() const override;
		};

		//Tracks a set of bundles and stats for a named session. This way you can have stats across multiple bundle installs grouped by some naming convention
		class FSessionPersistentStats : public FPersistentStatsBase
		{
		public:
			INSTALLBUNDLEMANAGER_API BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE("SessionName", SessionName);
				JSON_SERIALIZE_ARRAY("RequiredBundles", RequiredBundles);
				JSON_SERIALIZE("AnalyticsSessionID", AnalyticsSessionID);
				JSON_SERIALIZE_MAP("CountStats", CountStatMap);
				JSON_SERIALIZE_MAP_SERIALIZABLE("TimingStats", TimingStatsMap, FPersistentTimerData);
			END_JSON_SERIALIZER

		private:
			//Remove default constructor as we require a ContentRequestName in the constructor
			FSessionPersistentStats() = delete;

			//Used in our full path and should signify some identifier for a given ContentRequest
			FString SessionName;

			//Used to keep track of all bundles that are part of this content request
			TArray<FString> RequiredBundles;

		public:
			FSessionPersistentStats(const FString& ContentRequestNameIn)
				: SessionName(ContentRequestNameIn)
				, RequiredBundles()
			{
			}

			virtual ~FSessionPersistentStats() = default;

			INSTALLBUNDLEMANAGER_API void AddRequiredBundles(const TArray<FString>& RequiredBundlesToAdd);
			INSTALLBUNDLEMANAGER_API void AddRequiredBundles(const TArray<FName>& RequiredBundlesToAdd);

			//Removes all Required Bundle Names except the ones in the NewRequiredBundlesList. If no NewRequiredBundlesList is supplied, just erases all RequiredBundles.
			//This includes ones loaded from the persistent cache from previous runs. This re-saves the persistent cache file so they will be removed completely.
			INSTALLBUNDLEMANAGER_API void ResetRequiredBundles(const TArray<FString>& NewRequiredBundles = TArray<FString>());

			//Populates the passed in TArray with all the required bundles' names
			INSTALLBUNDLEMANAGER_API void GetRequiredBundles(TArray<FString>& OutRequiredBundles) const;
			
			//Implementation for FInstallBundleStatsContainerBase
		public:
			INSTALLBUNDLEMANAGER_API virtual const FString GetFullPathForStatFile() const override;
		};

		//Helper class to handle holding a collection of Bundle and Session stats.
		class FPersistentStatContainerBase
		{
		public:
			INSTALLBUNDLEMANAGER_API FPersistentStatContainerBase();
			INSTALLBUNDLEMANAGER_API virtual ~FPersistentStatContainerBase();

			//NOTE! These 2 functions are not virtual as we call them in the construct/deconstructor!
			INSTALLBUNDLEMANAGER_API void InitializeBase();
			INSTALLBUNDLEMANAGER_API void ShutdownBase();

			//Starts Bundle Persistent Stat Tracking for the given BundleName. Will automatically load applicable data from the disk and reset that cache
			//if the ExpectedAnalyticsID doesn't match the one previously serialized.
			//Uses FInstallBundleStatsContainerBase::GetBaseExpectedAnalyticsID if an ExpectedAnalyticsID is not passed in or empty
			INSTALLBUNDLEMANAGER_API virtual void StartBundlePersistentStatTracking(FName BundleName, const FString& ExpectedAnalyticsID = FString(), bool bForceResetStatData = false);
			
			//Starts Session Persistent Stat Tracking under the given SessionName. Will automatically load applicable data from the disk and reset that cache
			//if the ExpectedAnalyticsID doesn't match the one previously serialized. 
			//Uses FInstallBundleStatsContainerBase::GetBaseExpectedAnalyticsID if an ExpectedAnalyticsID is not passed in or empty
			INSTALLBUNDLEMANAGER_API virtual void StartSessionPersistentStatTracking(const FString& SessionName, const TArray<FName>& RequiredBundles = TArray<FName>(), const FString& ExpectedAnalyticsID = FString(), bool bForceResetStatData = false);

			INSTALLBUNDLEMANAGER_API virtual void StopBundlePersistentStatTracking(FName BundleName, bool bStopAllActiveTimers = true);
			INSTALLBUNDLEMANAGER_API virtual void StopSessionPersistentStatTracking(const FString& SessionName, bool bStopAllActiveTimers = true);

			INSTALLBUNDLEMANAGER_API virtual void StartBundlePersistentStatTimer(FName BundleName, ETimingStatNames TimerToStart);
			INSTALLBUNDLEMANAGER_API virtual void StartSessionPersistentStatTimer(const FString& SessionName, ETimingStatNames TimerToStart);

			INSTALLBUNDLEMANAGER_API virtual void StopBundlePersistentStatTimer(FName BundleName, ETimingStatNames TimerToStop);
			INSTALLBUNDLEMANAGER_API virtual void StopSessionPersistentStatTimer(const FString& SessionName, ETimingStatNames TimerToStop);

			INSTALLBUNDLEMANAGER_API virtual void UpdateBundlePersistentStatTimer(FName BundleName, ETimingStatNames TimerToUpdate);
			INSTALLBUNDLEMANAGER_API virtual void UpdateSessionPersistentStatTimer(const FString& SessionName, ETimingStatNames TimerToUpdate);

			INSTALLBUNDLEMANAGER_API virtual void IncrementBundlePersistentCounter(FName BundleName, ECountStatNames CounterToUpdate);
			INSTALLBUNDLEMANAGER_API virtual void IncrementSessionPersistentCounter(const FString& SessionName, ECountStatNames CounterToUpdate);

			INSTALLBUNDLEMANAGER_API virtual const FBundlePersistentStats* GetBundleStat(FName BundleName) const;
			INSTALLBUNDLEMANAGER_API virtual const FSessionPersistentStats* GetSessionStat(const FString& SessionName) const;

			INSTALLBUNDLEMANAGER_API virtual void SaveAllDirtyStatsToDisk();
			
			//Deletes persistent stat information from our stats for this Session/Bundle to reduce memory useage
			INSTALLBUNDLEMANAGER_API virtual void RemoveSessionStats(const FString& SessionName);
			INSTALLBUNDLEMANAGER_API virtual void RemoveBundleStats(FName BundleName);
			
		protected:
			INSTALLBUNDLEMANAGER_API virtual bool Tick(float dt);
			INSTALLBUNDLEMANAGER_API virtual void ResetTimerUpdate();
			INSTALLBUNDLEMANAGER_API virtual void ResetDirtyStatUpdate();

			INSTALLBUNDLEMANAGER_API virtual void UpdateAllBundlesActiveTimers();
			INSTALLBUNDLEMANAGER_API virtual void UpdateAllSessionActiveTimers();

			INSTALLBUNDLEMANAGER_API virtual void OnApp_EnteringBackground();
			INSTALLBUNDLEMANAGER_API virtual void OnApp_EnteringForeground();

			INSTALLBUNDLEMANAGER_API virtual void OnBackground_HandleBundleStats();
			INSTALLBUNDLEMANAGER_API virtual void OnForeground_HandleBundleStats();
			INSTALLBUNDLEMANAGER_API virtual void OnBackground_HandleSessionStats();
			INSTALLBUNDLEMANAGER_API virtual void OnForeground_HandleSessionStats();
			
			//Called whenever we start a timer
			//NOTE: doesn't get called by timer's being Start/Stopped for bShouldAutoHandleFGBGStats
			INSTALLBUNDLEMANAGER_API virtual void OnTimerStartedForStat(FPersistentStatsBase& BundleStatForTimer, ETimingStatNames TimerStarted);

			//Called whenever we stop a timer 
			//NOTE: doesn't get called by timer's being Start/Stopped for bShouldAutoHandleFGBGStats
			INSTALLBUNDLEMANAGER_API virtual void OnTimerStoppedForStat(FPersistentStatsBase& BundleStatForTimer, ETimingStatNames TimerStarted);

			//Stops active foreground timers when going to background and starts applicable background version
			//Also increments background counters
			INSTALLBUNDLEMANAGER_API virtual void UpdateStatsForBackground(FPersistentStatsBase& StatToUpdate);

			//Stops active background timers and resumes applicable foreground timers
			//Also increments foreground counters
			INSTALLBUNDLEMANAGER_API virtual void UpdateStatsForForeground(FPersistentStatsBase& StatToUpdate);
			
			//Goes through this Session's RequiredBundles and loads data from disk for those bundles. Allows us to make sure we have
			//full data in memory from disk without having to know the previous runs AnalyticsID to call StatsBegin on that bundle.
			//NOTE: You should still call StatsBegin on the listed bundles before Starting/Incrementing/ETC any analytics! This just
			//ensures that the previous run data exists for this given session!
			INSTALLBUNDLEMANAGER_API virtual void LoadRequiredBundleDataFromDiskForSession(const FString& SessionName);
			
		protected:
			TMap<FName, FBundlePersistentStats> PerBundlePersistentStatMap;
			TMap<FString, FSessionPersistentStats> SessionPersistentStatMap;

			FTSTicker::FDelegateHandle TickHandle;
			FDelegateHandle OnApp_EnteringForegroundHandle;
			FDelegateHandle OnApp_EnteringBackgroundHandle;

			//internal counter used to track when we need to run our timer update during tick. Requires bShouldAutoUpdateTimersInTick to be set to true to effect anything.
			//reset to TimerAutoUpdateRate whenever we trigger the auto update.
			float TimerAutoUpdateTimeRemaining;

			//internal counter used to track when we need to update all of our dirty stats. Requires bShouldSaveDirtyStatsOnTick to be set to true to effect anything.
			//reset to TimerAutoUpdateRate whenever we trigger the dirty stat update
			float TimerDirtyStatUpdateTimeRemaining;
			

			/****** Config Values ******/
			// All values are loaded from GEngineIni in [InstallBundleManager.PersistentStatSettings]

			/****** Timer Auto Update Settings ****/
			//These settings determine if we auto-update our Timer stats on a tick. Without an auto-update timers would only update on Start / Stop potentially leaving us vulnerable to
			//dropping a lot of timing information in the event of a crash. By setting these values you ensure your data is accurate to some interval.
			//
			//bShouldAutoUpdateTimersInTick -- Determines if the auto update feature is on at all. Rate of auto timer updates is determined by TimerAutoUpdateRate.
			//
			//TimerAutoUpdateRate -- Determines the rate we trigger the update in seconds. <=0 means update every tick.
			//
			bool bShouldAutoUpdateTimersInTick;
			float TimerAutoUpdateRate;

			/****** Stat Save To Disk Settings ****/
			//These settings determine how we handle saving our stats to disk. Outside of these settings, stats ALWAYS try to save to disk on Start or End, but these settings
			//let us also configure other times we may want to save the data to disk.
			//
			//bShouldSaveDirtyStatsOnTick -- If this is set to true then all stats that have bIsDirty set to true will try to save to disk every X seconds. Update rate determined by DirtyStatSaveToDiskRate
			//								 This way we will not update a given stat more then once a tick to disk.
			//
			//DirtyStatSaveToDiskRate -- Determines how often we update our dirty stats in seconds. <=0 means update every tick.
			bool bShouldSaveDirtyStatsOnTick;
			float DirtyStatSaveToDiskRate;

			//Determines if we should automatically Start,Stop,and Update the _FG and _BG versions of our timer stats.
			//If true you only have to call Start/Stop on the _Real versions of all timers and we will automatically
			//Start and Stop the _FG and _BG timers when going to background or returning to foreground to keep
			//their data accurate.
			bool bShouldAutoHandleFGBGStats;
		};
	}

	struct IBundleSourceContentRequestSharedContext
	{
		virtual ~IBundleSourceContentRequestSharedContext() = default;
	};

	struct FContentRequestSharedContext
	{
		FContentRequestSharedContext() = default;
		FContentRequestSharedContext(const FContentRequestSharedContext& Other) = delete;
		FContentRequestSharedContext(FContentRequestSharedContext&& Other) = default;
		FContentRequestSharedContext& operator=(const FContentRequestSharedContext& Other) = delete;
		FContentRequestSharedContext& operator=(FContentRequestSharedContext&& Other) = default;

		TMap<FInstallBundleSourceType, TUniquePtr<IBundleSourceContentRequestSharedContext>> BundleSourceSharedContext;

		// (Optional) - can be used to pass a list of requested bundles to IInstallBundleSource implementations
		TArray<FName> RequestedBundles;	
	};
	using FContentRequestSharedContextPtr = TSharedPtr<FContentRequestSharedContext>;
} // InstallBundleUtil
